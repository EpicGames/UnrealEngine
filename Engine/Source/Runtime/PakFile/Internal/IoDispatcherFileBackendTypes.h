// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IO/IoAllocators.h"
#include "IO/IoStore.h"
#include "IO/IoDispatcherBackend.h"
#include "IO/IoDispatcherFilesystemStats.h"
#include "Async/MappedFileHandle.h"
#include "Async/Mutex.h"
#include "Modules/ModuleInterface.h"

#include <atomic>

#ifndef PLATFORM_IMPLEMENTS_IO
#define PLATFORM_IMPLEMENTS_IO 0
#endif

#ifndef PLATFORM_IODISPATCHER_MODULE
#define PLATFORM_IODISPATCHER_MODULE UE_STRINGIZE(UE_JOIN(PLATFORM_HEADER_NAME, PlatformIoDispatcher))
#endif

#define UE_FILEIOSTORE_STATS_ENABLED (COUNTERSTRACE_ENABLED || CSV_PROFILER_STATS)

struct FFileIoStoreCompressionContext;

struct FFileIoStoreContainerFilePartition
{
	FFileIoStoreContainerFilePartition() = default;
	FFileIoStoreContainerFilePartition(FFileIoStoreContainerFilePartition&&) = delete;
	FFileIoStoreContainerFilePartition(const FFileIoStoreContainerFilePartition&) = delete;
	~FFileIoStoreContainerFilePartition();

	FFileIoStoreContainerFilePartition& operator=(FFileIoStoreContainerFilePartition&&) = delete;
	FFileIoStoreContainerFilePartition& operator=(const FFileIoStoreContainerFilePartition&) = delete;

	// This will lazily map the file, if needed. It is thread-safe to borrow
	// handles, but their lifetimes are tied to this partition.
	IMappedFileHandle* BorrowMappedFileHandle();

	FString FilePath;
	uint64 FileHandle = 0;
	uint64 FileSize = 0;
	uint32 ContainerFileIndex = 0;
	std::atomic<int32> StartedReadRequestsCount = 0;

private:	
	std::atomic<IMappedFileHandle*> MappedFileHandle = nullptr;
	UE::FMutex MappedFileHandleMutex;
};

struct FFileIoStoreContainerFile
{
	FFileIoStoreContainerFile() = default;
	UE_NONCOPYABLE(FFileIoStoreContainerFile);

	uint64 PartitionSize = 0;
	uint64 CompressionBlockSize = 0;
	TArray<FName> CompressionMethods;
	TConstArrayView<FIoStoreTocCompressedBlockEntry> CompressionBlocks;
	FString FilePath;
	FGuid EncryptionKeyGuid;
	FAES::FAESKey EncryptionKey;
	EIoContainerFlags ContainerFlags;
	TConstArrayView<FSHAHash> BlockSignatureTable;
	TArray<FFileIoStoreContainerFilePartition> Partitions;
	uint32 ContainerInstanceId = 0;
	std::atomic_int32_t RefCount{0};

	void GetPartitionAndOffset(uint64 TocOffset, FFileIoStoreContainerFilePartition*& OutPartition, uint64& OutOffset)
	{
		int32 PartitionIndex = int32(TocOffset / PartitionSize);
		OutPartition = &Partitions[PartitionIndex];
		OutOffset = TocOffset % PartitionSize;
	}

	void Empty()
	{
		PartitionSize = 0;
		CompressionBlockSize = 0;
		CompressionMethods.Empty();
		CompressionBlocks = TConstArrayView<FIoStoreTocCompressedBlockEntry>();
		FilePath.Empty();
		EncryptionKeyGuid = FGuid();
		EncryptionKey = FAES::FAESKey();
		ContainerFlags = EIoContainerFlags::None;
		BlockSignatureTable = TConstArrayView<FSHAHash>();
		Partitions.Empty();
		ContainerInstanceId = 0;
	}
};

struct FFileIoStoreBuffer
{
	FFileIoStoreBuffer* Next = nullptr;
	uint8* Memory = nullptr;
};

struct FFileIoStoreBlockKey
{
	union
	{
		struct
		{
			uint32 FileIndex;
			uint32 BlockIndex;
		};
		uint64 Hash;
	};


	friend bool operator==(const FFileIoStoreBlockKey& A, const FFileIoStoreBlockKey& B)
	{
		return A.Hash == B.Hash;
	}

	friend uint32 GetTypeHash(const FFileIoStoreBlockKey& Key)
	{
		return GetTypeHash(Key.Hash);
	}
};

struct FFileIoStoreBlockScatter
{
	struct FFileIoStoreResolvedRequest* Request = nullptr;
	uint64 DstOffset = 0;
	uint64 SrcOffset = 0;
	uint64 Size = 0;
};

struct FFileIoStoreCompressedBlock
{
	FFileIoStoreCompressedBlock* Next = nullptr;
	FFileIoStoreBlockKey Key;
	FName CompressionMethod;
	uint64 RawOffset = uint64(-1);
	uint32 UncompressedSize = uint32(-1);
	uint32 CompressedSize = uint32(-1);
	uint32 RawSize = uint32(-1);
	uint32 RefCount = 0;
	uint32 UnfinishedRawBlocksCount = 0;
	TArray<struct FFileIoStoreReadRequest*, TInlineAllocator<2>> RawBlocks;
	TArray<FFileIoStoreBlockScatter, TInlineAllocator<2>> ScatterList;
	FFileIoStoreCompressionContext* CompressionContext = nullptr;
	uint8* CompressedDataBuffer = nullptr;
	FAES::FAESKey EncryptionKey;
	TConstArrayView<FSHAHash> BlockSignatureTable;
	const FSHAHash* SignatureHash = nullptr;
	bool bFailed = false;
	bool bCancelled = false;
};

struct FFileIoStoreReadRequest
{
	enum EQueueStatus
	{
		QueueStatus_NotInQueue,
		QueueStatus_InQueue,
		QueueStatus_Started,
		QueueStatus_Completed
	};

	FFileIoStoreReadRequest()
		: Sequence(NextSequence++)
		, CreationTime(FPlatformTime::Cycles64())
	{
	}
	FFileIoStoreReadRequest* Next = nullptr;
	FFileIoStoreReadRequest* Previous = nullptr;
	FFileIoStoreContainerFilePartition* ContainerFilePartition = nullptr;
	uint64 Offset = uint64(-1);
	uint64 Size = uint64(-1);
	FFileIoStoreBlockKey Key;
	FFileIoStoreBuffer* Buffer = nullptr;
	uint32 RefCount = 0;
	uint32 BufferRefCount = 0;
	TArray<FFileIoStoreCompressedBlock*, TInlineAllocator<8>> CompressedBlocks;
	const uint32 Sequence;
	int32 Priority = 0;
	uint64 CreationTime;	// Potentially used to circuit break request ordering optimizations when outstanding requests have been delayed too long
	FFileIoStoreBlockScatter ImmediateScatter;
	uint32 BytesUsed = 0;
	bool bIsCustomRequest = false;
	bool bFailed = false;
	bool bCancelled = false;
	EQueueStatus QueueStatus = QueueStatus_NotInQueue;

#if DO_CHECK
	// For debug checks that we are in the correct owning list for our intrusive next/previous pointers
	uint32 ListCookie = 0;
#endif

private:
	static PAKFILE_API uint32 NextSequence;
};

#define CHECK_IO_STORE_READ_REQUEST_LIST_MEMBERSHIP (DO_CHECK && 0)
// Iterator class for traversing and emptying a list FFileIoStoreReadRequestList at the same time
class FFileIoStoreReadRequestListStealingIterator
{
public:
	FFileIoStoreReadRequestListStealingIterator(const FFileIoStoreReadRequestListStealingIterator&) = delete;
	FFileIoStoreReadRequestListStealingIterator& operator=(const FFileIoStoreReadRequestListStealingIterator&) = delete;

	FFileIoStoreReadRequestListStealingIterator(FFileIoStoreReadRequestListStealingIterator&& Other)
	{
		Current = Other.Current;
		Next = Other.Next;

		Other.Current = Other.Next = nullptr;
	}

	FFileIoStoreReadRequestListStealingIterator& operator=(FFileIoStoreReadRequestListStealingIterator&& Other)
	{
		Current = Other.Current;
		Next = Other.Next;

		Other.Current = Other.Next = nullptr;

		return *this;
	}

	void operator++()
	{
		AdvanceTo(Next);
	}

	FFileIoStoreReadRequest* operator*()
	{
		return Current;
	}

	FFileIoStoreReadRequest* operator->()
	{
		return Current;
	}

	explicit operator bool() const
	{
		return Current != nullptr;
	}

private:
	friend class FFileIoStoreReadRequestList; // Only the list can construct us

	FFileIoStoreReadRequestListStealingIterator(FFileIoStoreReadRequest* InHead)
	{
#if CHECK_IO_STORE_READ_REQUEST_LIST_MEMBERSHIP
		// None of these nodes are members of the list any more
		for (FFileIoStoreReadRequest* Cursor = InHead; Cursor; Cursor = Cursor->Next)
		{
			Cursor->ListCookie = 0;
		}
#endif

		AdvanceTo(InHead);
	}

	void AdvanceTo(FFileIoStoreReadRequest* NewCurrent)
	{
		Current = NewCurrent;
		if (Current)
		{
			// Copy off the next ptr and remove Current from its list so it can safely be added to another list
			Next = Current->Next;
			Current->Next = nullptr;
		}
		else
		{
			Next = nullptr;
		}
	}

	FFileIoStoreReadRequest* Current = nullptr;
	FFileIoStoreReadRequest* Next = nullptr;;
};

class FFileIoStoreReadRequestListIterator
{
public:
	FFileIoStoreReadRequestListIterator(const FFileIoStoreReadRequestListIterator&) = default;
	FFileIoStoreReadRequestListIterator& operator=(const FFileIoStoreReadRequestListIterator&) = default;

	void operator++()
	{
		checkf(Current->Next == Next, TEXT("FFileIoStoreReadRequestListIterator cached Next pointer != Current->Next - was list modified during iteration?"));
		AdvanceTo(Next);
	}

	FFileIoStoreReadRequest* operator*()
	{
		return Current;
	}

	FFileIoStoreReadRequest* operator->()
	{
		return Current;
	}

	explicit operator bool() const
	{
		return Current != nullptr;
	}

	bool operator!=(const FFileIoStoreReadRequestListIterator& Other) const
	{
		return Current != Other.Current;
	}

private:
	friend class FFileIoStoreReadRequestList; // Only the list can construct us

	FFileIoStoreReadRequestListIterator(FFileIoStoreReadRequest* InHead)
	{
		AdvanceTo(InHead);
	}

	void AdvanceTo(FFileIoStoreReadRequest* NewCurrent)
	{
		Current = NewCurrent;

		if (Current)
		{
			// Copy off the next ptr to check for edits made to this list during iteration
			Next = Current->Next;
		}
		else
		{
			Next = nullptr;
		}
	}

	FFileIoStoreReadRequest* Current = nullptr;
	FFileIoStoreReadRequest* Next = nullptr;
};


// Wrapper for doubly-linked intrusive list of FFileIoStoreReadRequest

class FFileIoStoreReadRequestList
{
public:
	FFileIoStoreReadRequestList()
#if CHECK_IO_STORE_READ_REQUEST_LIST_MEMBERSHIP
		: ListCookie(++NextListCookie)
#endif
	{
	}

	// Owns the Next/Previous pointers of the FFileIoStoreReadRequests it contains so it can't be copied
	FFileIoStoreReadRequestList(const FFileIoStoreReadRequestList& Other) = delete;
	FFileIoStoreReadRequestList& operator=(const FFileIoStoreReadRequestList& Other) = delete;

	FFileIoStoreReadRequestList(FFileIoStoreReadRequestList&& Other)
	{
		Head = Other.Head;
		Tail = Other.Tail;
		Other.Head = Other.Tail = nullptr;

#if CHECK_IO_STORE_READ_REQUEST_LIST_MEMBERSHIP
		for (FFileIoStoreReadRequest* Cursor = Head; Cursor; Cursor = Cursor->Next)
		{
			Cursor->ListCookie = ListCookie;
		}
#endif
	}

	FFileIoStoreReadRequestList& operator=(FFileIoStoreReadRequestList&& Other)
	{
		check(!Head && !Tail);

		Head = Other.Head;
		Tail = Other.Tail;
		Other.Head = Other.Tail = nullptr;

#if CHECK_IO_STORE_READ_REQUEST_LIST_MEMBERSHIP
		for (FFileIoStoreReadRequest* Cursor = Head; Cursor; Cursor = Cursor->Next)
		{
			Cursor->ListCookie = ListCookie;
		}
#endif

		return *this;
	}

#if CHECK_IO_STORE_READ_REQUEST_LIST_MEMBERSHIP
	~FFileIoStoreReadRequestList()
	{
		// If anything is left in this list it should still think we own it
		for (FFileIoStoreReadRequest* Cursor = Head; Cursor; Cursor = Cursor->Next)
		{
			check(Cursor->ListCookie == ListCookie);
		}
	}
#endif

	bool IsEmpty() const
	{
		return Head == nullptr;
	}

	// Steal the whole list for iteration and moving the contents to other lists
	FFileIoStoreReadRequestListStealingIterator Steal()
	{
#if CHECK_IO_STORE_READ_REQUEST_LIST_MEMBERSHIP
		for (FFileIoStoreReadRequest* Cursor = Head; Cursor; Cursor = Cursor->Next)
		{
			check(Cursor->ListCookie == ListCookie);
		}
#endif

		FFileIoStoreReadRequest* OldHead = Head;
		Clear();
		return FFileIoStoreReadRequestListStealingIterator(OldHead);
	}

	// Non-stealing iterator
	FFileIoStoreReadRequestListIterator CreateIterator() const
	{
		return FFileIoStoreReadRequestListIterator(Head);
	}

	FFileIoStoreReadRequestListIterator begin() const {
		return CreateIterator();
	}
	FFileIoStoreReadRequestListIterator end() const {
		return FFileIoStoreReadRequestListIterator(nullptr);
	}

	FFileIoStoreReadRequest* PeekHead() const
	{
		return Head;
	}

	void Add(FFileIoStoreReadRequest* Request)
	{
#if CHECK_IO_STORE_READ_REQUEST_LIST_MEMBERSHIP
		check(Request->ListCookie == 0);
		Request->ListCookie = ListCookie;
#endif

		if (Tail)
		{
			Tail->Next = Request;
			Request->Previous = Tail;
		}
		else
		{
			Head = Request;
			Request->Previous = nullptr;
		}
		Tail = Request;
		Request->Next = nullptr;
	}


	// Remove all FFileIoStoreReadRequests from List and add them to this list
	void AppendSteal(FFileIoStoreReadRequestList& List)
	{
		if (List.Head)
		{
			FFileIoStoreReadRequest *ListHead = List.Head, *ListTail = List.Tail;
			List.Clear();
			AppendSteal(ListHead, ListTail);
		}
	}

	void Remove(FFileIoStoreReadRequest* Request)
	{
#if CHECK_IO_STORE_READ_REQUEST_LIST_MEMBERSHIP
		check(Request);
		check(Request->ListCookie == ListCookie);
		Request->ListCookie = 0;
#endif

		if (Head == Request && Tail == Request)
		{
			check(Request->Next == nullptr);
			check(Request->Previous == nullptr);

			Head = Tail = nullptr;
		}
		else if (Head == Request)
		{
			check(Request->Previous == nullptr);

			Head = Request->Next;
			Head->Previous = nullptr;
			Request->Next = nullptr;
		}
		else if (Tail == Request)
		{
			check(Request->Next == nullptr);

			Tail = Request->Previous;
			Tail->Next = nullptr;
			Request->Previous = nullptr;
		}
		else
		{
			check(Request->Next != nullptr && Request->Previous != nullptr); // Neither head nor tail should mean both links are live

			Request->Next->Previous = Request->Previous;
			Request->Previous->Next = Request->Next;

			Request->Next = Request->Previous = nullptr;
		}
	}

	void Clear()
	{
#if CHECK_IO_STORE_READ_REQUEST_LIST_MEMBERSHIP
		for (FFileIoStoreReadRequest* Cursor = Head; Cursor; Cursor = Cursor->Next)
		{
			check(Cursor->ListCookie == ListCookie);
			Cursor->ListCookie = 0;
		}
#endif
		Head = Tail = nullptr;
	}

private:
	FFileIoStoreReadRequest* Head = nullptr;
	FFileIoStoreReadRequest* Tail = nullptr;

#if CHECK_IO_STORE_READ_REQUEST_LIST_MEMBERSHIP
	uint32 ListCookie;
	static PAKFILE_API uint32 NextListCookie;
#endif
	
	void AppendSteal(FFileIoStoreReadRequest* ListHead, FFileIoStoreReadRequest* ListTail)
	{
		check(ListHead);
		check(ListTail);
		check(!ListTail->Next);
		check(!ListHead->Previous);
		check(ListTail == ListHead || ListTail->Previous != nullptr);
		check(ListTail == ListHead || ListHead->Next != nullptr);

#if CHECK_IO_STORE_READ_REQUEST_LIST_MEMBERSHIP
		for (FFileIoStoreReadRequest* Cursor = ListHead; Cursor; Cursor = Cursor->Next)
		{
			check(Cursor->ListCookie == 0);
			Cursor->ListCookie = ListCookie;
		}
#endif

		if (Tail)
		{
			Tail->Next = ListHead;
			ListHead->Previous = Tail;
		}
		else
		{
			Head = ListHead;
			ListHead->Previous = nullptr;
		}
		Tail = ListTail;
	}
};

class FFileIoStoreStats;

class FFileIoStoreBufferAllocator
{
public:
	FFileIoStoreBufferAllocator(FFileIoStoreStats& InStats)
		: Stats(InStats)
	{
	}

	PAKFILE_API void Initialize(uint64 MemorySize, uint64 BufferSize, uint32 BufferAlignment);
	PAKFILE_API FFileIoStoreBuffer* AllocBuffer();
	PAKFILE_API void FreeBuffer(FFileIoStoreBuffer* Buffer);
	uint64 GetBufferSize() const { return BufferSize; }

private:
	FFileIoStoreStats& Stats;
	uint64 BufferSize = 0;
	uint8* BufferMemory = nullptr;
	FCriticalSection BuffersCritical;
	FFileIoStoreBuffer* FirstFreeBuffer = nullptr;
};

class FFileIoStoreBlockCache
{
public:
	PAKFILE_API FFileIoStoreBlockCache(FFileIoStoreStats& Stats);
	PAKFILE_API ~FFileIoStoreBlockCache();

	PAKFILE_API void Initialize(uint64 CacheMemorySize, uint64 ReadBufferSize);
	PAKFILE_API bool Read(FFileIoStoreReadRequest* Block);
	PAKFILE_API void Store(const FFileIoStoreReadRequest* Block);

private:
	struct FCachedBlock
	{
		FCachedBlock* LruPrev = nullptr;
		FCachedBlock* LruNext = nullptr;
		uint64 Key = 0;
		uint8* Buffer = nullptr;
	};

	FFileIoStoreStats& Stats;
	uint8* CacheMemory = nullptr;
	TMap<uint64, FCachedBlock*> CachedBlocks;
	FCachedBlock CacheLruHead;
	FCachedBlock CacheLruTail;
	uint64 ReadBufferSize = 0;
};

struct FFileIoStoreReadRequestSortKey
{
	uint64 Offset = 0;
	uint64 Handle = 0;
	int32 Priority = 0;

	FFileIoStoreReadRequestSortKey() {}
	FFileIoStoreReadRequestSortKey(FFileIoStoreReadRequest* Request)
		: Offset(Request->Offset), Handle(Request->ContainerFilePartition->FileHandle), Priority(Request->Priority)
	{
	}
};

// Stores FFileIoStoreReadRequest sorted by file handle & offset with a parallel list sorted by insertion order 
class FFileIoStoreOffsetSortedRequestQueue
{
public:
	PAKFILE_API FFileIoStoreOffsetSortedRequestQueue(int32 InPriority);
	FFileIoStoreOffsetSortedRequestQueue(const FFileIoStoreOffsetSortedRequestQueue&) = delete;
	FFileIoStoreOffsetSortedRequestQueue(FFileIoStoreOffsetSortedRequestQueue&&) = default;
	FFileIoStoreOffsetSortedRequestQueue& operator=(const FFileIoStoreOffsetSortedRequestQueue&) = delete;
	FFileIoStoreOffsetSortedRequestQueue& operator=(FFileIoStoreOffsetSortedRequestQueue&&) = default;

	int32 GetPriority() const { return Priority; }
	bool IsEmpty() const { return Requests.Num() == 0; }
	
	// Remove all requests from this container and return them, for switching to a different queue scheme
	PAKFILE_API TArray<FFileIoStoreReadRequest*> StealRequests();
	// Remove all requests whose priority has been changed to something other than the Priority of this queue
	PAKFILE_API TArray<FFileIoStoreReadRequest*> RemoveMisprioritizedRequests();
	PAKFILE_API void RemoveCancelledRequests(TArray<FFileIoStoreReadRequest*>& OutCancelled);

	PAKFILE_API FFileIoStoreReadRequest* Pop(FFileIoStoreReadRequestSortKey LastSortKey);
	PAKFILE_API void Push(FFileIoStoreReadRequest* Request);
	PAKFILE_API int32 HandleContainerUnmounted(const FFileIoStoreContainerFile& ContainerFile);

private:
	int32 Priority;
	int32 PeekRequestIndex = INDEX_NONE;
	
	// Requests sorted by file handle & offset
	TArray<FFileIoStoreReadRequest*> Requests;
	
	// Requests sorted by insertion order 
	// We store this on the heap in case we get moved, FFileIoStoreReadRequest keeps pointers to FFileIoStoreReadRequestList for debugging
	FFileIoStoreReadRequestList RequestsBySequence;

	PAKFILE_API FFileIoStoreReadRequest* GetNextInternal(FFileIoStoreReadRequestSortKey LastSortKey, bool bPop);

	static FFileIoStoreReadRequestSortKey RequestSortProjection(FFileIoStoreReadRequest* Request) { return FFileIoStoreReadRequestSortKey(Request); }
	static PAKFILE_API bool RequestSortPredicate(const FFileIoStoreReadRequestSortKey& A, const FFileIoStoreReadRequestSortKey& B);
};

class FFileIoStoreRequestQueue
{
public:
	PAKFILE_API FFileIoStoreReadRequest* Pop();
	PAKFILE_API void PopCancelled(TArray<FFileIoStoreReadRequest*>& OutCancelled);
	PAKFILE_API void Push(FFileIoStoreReadRequest& Request);	// Takes ownership of Request and rewrites its intrustive linked list pointers
	PAKFILE_API void Push(FFileIoStoreReadRequestList& Requests); // Consumes the request list and overwrites all intrustive linked list pointers
	PAKFILE_API void UpdateOrder();
	PAKFILE_API void Lock();
	PAKFILE_API void Unlock();
	PAKFILE_API int32 HandleContainerUnmounted(const FFileIoStoreContainerFile& ContainerFile);

private:
	static bool QueueSortFunc(const FFileIoStoreReadRequest& A, const FFileIoStoreReadRequest& B)
	{
		if (A.Priority == B.Priority)
		{
			return A.Sequence < B.Sequence;
		}
		return A.Priority > B.Priority;
	}
	void UpdateSortRequestsByOffset(); // Check if we need to switch sorting schemes based on the CVar
	void PushToPriorityQueues(FFileIoStoreReadRequest* Request);
	static int32 QueuePriorityProjection(const FFileIoStoreOffsetSortedRequestQueue& A) { return A.GetPriority(); }

	bool bSortRequestsByOffset = false; // Cached value of CVar controlling whether we use Heap or SortedPriorityQueues
	
	// Heap sorted by request order
	TArray<FFileIoStoreReadRequest*> Heap;
	FCriticalSection CriticalSection;

	// Queues sorted by increasing priority
	TArray<FFileIoStoreOffsetSortedRequestQueue> SortedPriorityQueues; 
	// The last offset, file handle and priority we popped so that we can pop the closest forward read for the next IO operation
	FFileIoStoreReadRequestSortKey LastSortKey;

#if !UE_BUILD_SHIPPING
	TMap<int32, uint32> RequestPriorityCounts;
#endif
};

template <typename T, uint16 SlabSize = 4096>
using TIoDispatcherSingleThreadedSlabAllocator = TSingleThreadedSlabAllocator<T, SlabSize>;

struct FFileIoStoreReadRequestLink
{
	FFileIoStoreReadRequestLink(FFileIoStoreReadRequest& InReadRequest)
		: ReadRequest(InReadRequest)
	{
	}

	FFileIoStoreReadRequestLink* Next = nullptr;
	FFileIoStoreReadRequest& ReadRequest;
};

class FFileIoStoreRequestAllocator
{
public:
	int64 GetLiveReadRequestsCount() const
	{
		return LiveReadRequestsCount;
	}


	FFileIoStoreResolvedRequest* AllocResolvedRequest(
		FIoRequestImpl& InDispatcherRequest,
		FFileIoStoreContainerFile* InContainerFile,
		uint64 InResolvedOffset,
		uint64 InResolvedSize,
		int32 InPriority)
	{
		return ResolvedRequestAllocator.Construct(
			InDispatcherRequest,
			InContainerFile,
			InResolvedOffset,
			InResolvedSize,
			InPriority);
	}

	void Free(FFileIoStoreResolvedRequest* ResolvedRequest)
	{
		ResolvedRequestAllocator.Destroy(ResolvedRequest);
	}

	FFileIoStoreReadRequest* AllocReadRequest()
	{
		//TRACE_CPUPROFILER_EVENT_SCOPE(AllocReadRequest);
		++LiveReadRequestsCount;
		return ReadRequestAllocator.Construct();
	}

	void Free(FFileIoStoreReadRequest* ReadRequest)
	{
		//TRACE_CPUPROFILER_EVENT_SCOPE(FreeReadRequest);
		ReadRequestAllocator.Destroy(ReadRequest);
		--LiveReadRequestsCount;
	}

	FFileIoStoreCompressedBlock* AllocCompressedBlock()
	{
		//TRACE_CPUPROFILER_EVENT_SCOPE(AllocCompressedBlock);
		return CompressedBlockAllocator.Construct();
	}

	void Free(FFileIoStoreCompressedBlock* CompressedBlock)
	{
		//TRACE_CPUPROFILER_EVENT_SCOPE(FreeCompressedBlock);
		CompressedBlockAllocator.Destroy(CompressedBlock);
	}

	FFileIoStoreReadRequestLink* AllocRequestLink(FFileIoStoreReadRequest* ReadRequest)
	{
		check(ReadRequest);
		return RequestLinkAllocator.Construct(*ReadRequest);
	}

	void Free(FFileIoStoreReadRequestLink* RequestLink)
	{
		RequestLinkAllocator.Destroy(RequestLink);
	}

private:
	TIoDispatcherSingleThreadedSlabAllocator<FFileIoStoreResolvedRequest> ResolvedRequestAllocator;
	TIoDispatcherSingleThreadedSlabAllocator<FFileIoStoreReadRequest> ReadRequestAllocator;
	TIoDispatcherSingleThreadedSlabAllocator<FFileIoStoreCompressedBlock> CompressedBlockAllocator;
	TIoDispatcherSingleThreadedSlabAllocator<FFileIoStoreReadRequestLink> RequestLinkAllocator;
	int64 LiveReadRequestsCount = 0;
};

struct FFileIoStoreResolvedRequest
{
public:
	PAKFILE_API FFileIoStoreResolvedRequest(
		FIoRequestImpl& InDispatcherRequest,
		FFileIoStoreContainerFile* InContainerFile,
		uint64 InResolvedOffset,
		uint64 InResolvedSize,
		int32 InPriority);

	const FFileIoStoreContainerFile* GetContainerFile() const
	{
		return ContainerFile;
	}

	FFileIoStoreContainerFile* GetContainerFile()
	{
		return ContainerFile;
	}

	uint64 GetResolvedOffset() const
	{
		return ResolvedOffset;
	}

	uint64 GetResolvedSize() const
	{
		return ResolvedSize;
	}

	int32 GetPriority() const
	{
		return Priority;
	}

	bool HasBuffer() const
	{
		check(DispatcherRequest);
		return DispatcherRequest->HasBuffer();
	}

	void CreateBuffer(uint64 Size)
	{
		check(DispatcherRequest);
		DispatcherRequest->CreateBuffer(Size);
	}

	FIoBuffer& GetBuffer()
	{
		check(DispatcherRequest);
		return DispatcherRequest->GetBuffer();
	}

	PAKFILE_API void AddReadRequestLink(FFileIoStoreReadRequestLink* ReadRequestLink);

private:
	FIoRequestImpl* DispatcherRequest = nullptr;
	FFileIoStoreContainerFile* ContainerFile;
	FFileIoStoreReadRequestLink* ReadRequestsHead = nullptr;
	FFileIoStoreReadRequestLink* ReadRequestsTail = nullptr;
	const uint64 ResolvedOffset;
	const uint64 ResolvedSize;
	int32 Priority = 0;
	uint32 UnfinishedReadsCount = 0;
	bool bFailed = false;
	bool bCancelled = false;

	friend class FFileIoStore;
	friend class FFileIoStoreRequestTracker;
};


// Wrapper for sending stats to both insights and csv profiling from any platform-specific dispatcher.
#if UE_FILEIOSTORE_STATS_ENABLED
class FFileIoStoreStats
{
public:
	PAKFILE_API FFileIoStoreStats();
	PAKFILE_API ~FFileIoStoreStats();

	// Called by the backend when underlying file system reads start
	PAKFILE_API void OnFilesystemReadStarted(const FFileIoStoreReadRequest* Request);
	PAKFILE_API void OnFilesystemReadsStarted(const FFileIoStoreReadRequestList& Requests);
	// Called by the backend when underlying filesystem reads complete, possibly already decompressed on some systems
	PAKFILE_API void OnFilesystemReadCompleted(const FFileIoStoreReadRequest* Request);
	PAKFILE_API void OnFilesystemReadsCompleted(const FFileIoStoreReadRequestList& CompletedRequests);

private:
	friend class FFileIoStore;
	friend class FFileIoStoreBlockCache;
	friend class FFileIoStoreReader;
	friend class FFileIoStoreBufferAllocator;

	PAKFILE_API void OnReadRequestsQueued(const FFileIoStoreReadRequestList& Requests);
	PAKFILE_API void OnReadRequestsCompleted(const FFileIoStoreReadRequestList& Requests);
	PAKFILE_API void OnDecompressQueued(const FFileIoStoreCompressedBlock* CompressedBlock);
	PAKFILE_API void OnDecompressComplete(const FFileIoStoreCompressedBlock* CompressedBlock);

	// Bytes were copied into the buffer provided to users of the io system
	PAKFILE_API void OnBytesScattered(int64 BytesScattered);
	// Record stats for block cache hit/miss rate & data throughput
	PAKFILE_API void OnBlockCacheStore(uint64 NumBytes);
	PAKFILE_API void OnBlockCacheHit(uint64 NumBytes);
	PAKFILE_API void OnBlockCacheMiss(uint64 NumBytes);

	// A read was started without seeking
	PAKFILE_API void OnSequentialRead();
	// A read was started and was either a forward or reverse seek
	PAKFILE_API void OnSeek(uint64 LastOffset, uint64 NewOffset);
	// A read was started from a different file handle from the last one we used
	PAKFILE_API void OnHandleChangeSeek();

	PAKFILE_API void OnTocMounted(uint64 AllocatedSize);
	PAKFILE_API void OnTocUnmounted(uint64 AllocatedSize);

	PAKFILE_API void OnBufferReleased();
	PAKFILE_API void OnBufferAllocated();

	UE::FIoDispatcherFilesystemStats Stats;
};
#else
class FFileIoStoreStats
{
public:
	void OnFilesystemReadStarted(const FFileIoStoreReadRequest* Request) {}
	void OnFilesystemReadsStarted(const FFileIoStoreReadRequestList& Requests) {}
	void OnFilesystemReadCompleted(const FFileIoStoreReadRequest* Request) {}
	void OnFilesystemReadsCompleted(const FFileIoStoreReadRequestList& CompletedRequests) {}

private:
	friend class FFileIoStore;
	friend class FFileIoStoreBlockCache;
	friend class FFileIoStoreReader;
	friend class FFileIoStoreBufferAllocator;

	void OnReadRequestsQueued(const FFileIoStoreReadRequestList& Requests) {}
	void OnReadRequestsCompleted(const FFileIoStoreReadRequestList& Requests) {}
	void OnDecompressQueued(const FFileIoStoreCompressedBlock* CompressedBlock) {}
	void OnDecompressComplete(const FFileIoStoreCompressedBlock* CompressedBlock) {}
	void OnBytesScattered(int64 BytesScattered) {}
	void OnBlockCacheStore(uint64 NumBytes) {}
	void OnBlockCacheHit(uint64 NumBytes) {}
	void OnBlockCacheMiss(uint64 NumBytes) {}
	void OnTocMounted(uint64 AllocatedSize) {}
	void OnTocUnmounted(uint64 AllocatedSize) {}
	void OnBufferReleased() {}
	void OnBufferAllocated() {}
};
#endif

struct FInitializePlatformFileIoStoreParams
{
	const FWakeUpIoDispatcherThreadDelegate* WakeUpDispatcherThreadDelegate = nullptr;
	FFileIoStoreRequestAllocator* RequestAllocator = nullptr;
	FFileIoStoreBufferAllocator* BufferAllocator = nullptr;
	FFileIoStoreBlockCache* BlockCache = nullptr;
	FFileIoStoreStats* Stats = nullptr;
};

class IPlatformFileIoStore
{
public:
	virtual ~IPlatformFileIoStore() = default;
	virtual void Initialize(const FInitializePlatformFileIoStoreParams& Params) = 0;

	virtual bool OpenContainer(const TCHAR* ContainerFilePath, uint64& ContainerFileHandle, uint64& ContainerFileSize) = 0;
	virtual void CloseContainer(uint64 ContainerFileHandle) = 0;

	virtual bool CreateCustomRequests(FFileIoStoreResolvedRequest& ResolvedRequest, FFileIoStoreReadRequestList& OutRequests) = 0;
	virtual bool StartRequests(FFileIoStoreRequestQueue& RequestQueue) = 0;
	virtual void GetCompletedRequests(FFileIoStoreReadRequestList& OutRequests) = 0;

	virtual void ServiceNotify() = 0;
	virtual void ServiceWait() = 0;
};

class IPlatformFileIoStoreModule : public IModuleInterface
{
public:
	virtual TUniquePtr<IPlatformFileIoStore> CreatePlatformFileIoStore() = 0;
};

#if PLATFORM_IMPLEMENTS_IO
PAKFILE_API TUniquePtr<IPlatformFileIoStore> CreatePlatformFileIoStore();
#endif

