// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class PakFile : ModuleRules
{
	public PakFile(ReadOnlyTargetRules Target) : base(Target)
	{
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
		PrivateDependencyModuleNames.Add("Core");
		PrivateDependencyModuleNames.Add("CoreUObject");
		PrivateDependencyModuleNames.Add("TraceLog");
		PublicDependencyModuleNames.Add("RSA");
	}
}
