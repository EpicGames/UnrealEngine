// Copyright Epic Games, Inc. All Rights Reserved.

#include "FileIoDispatcherBackend.h"

#include "Algo/BinarySearch.h"
#include "Async/Mutex.h"
#include "Async/UniqueLock.h"
#include "HAL/CriticalSection.h"
#include "HAL/Platform.h"
#include "HAL/PlatformFileManager.h"
#include "IO/IoAllocators.h"
#include "IO/IoChunkId.h"
#include "IO/IoChunkEncoding.h"
#include "IO/IoContainerHeader.h"
#include "IO/IoContainerId.h"
#include "IO/IoDispatcher.h"
#include "IO/IoDispatcherBackend.h"
#include "IO/IoOffsetLength.h"
#include "IO/IoStatus.h"
#include "IO/IoStore.h"
#include "IO/PlatformIoDispatcher.h"
#include "Math/NumericLimits.h"
#include "Memory/MemoryView.h"
#include "Misc/EncryptionKeyManager.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/PathViews.h"
#include "Misc/ScopeLock.h"
#include "Misc/ScopeRWLock.h"
#include "Misc/StringBuilder.h"
#include "Serialization/MemoryReader.h"

#include <atomic>

namespace UE::IoStore
{

////////////////////////////////////////////////////////////////////////////////
namespace Config
{

float FileIoStoreUnmountTimeOutSeconds = 10.0f;
static FAutoConsoleVariableRef CVar_UnmountTimeOutSeconds(
	TEXT("fileiostore.UnmountTimeOutSeconds"),
	FileIoStoreUnmountTimeOutSeconds,
	TEXT("Max time to wait for pending I/O requests before unmounting a container.")
);

} // namespace Config

////////////////////////////////////////////////////////////////////////////////
class FMappedFileProxy final
	: public IMappedFileHandle
{
public:
	FMappedFileProxy(IMappedFileHandle* InSharedMappedFileHandle, uint64 InFileSize, int64 InBlockOffset)
		: IMappedFileHandle(InFileSize)
		, SharedMappedFileHandle(InSharedMappedFileHandle)
		, BlockOffset(InBlockOffset)
	{
	}

	virtual ~FMappedFileProxy() { }

	virtual IMappedFileRegion* MapRegion(int64 Offset = 0, int64 BytesToMap = MAX_int64, FFileMappingFlags Flags = EMappedFileFlags::ENone) override
	{
		return SharedMappedFileHandle != nullptr ? SharedMappedFileHandle->MapRegion(BlockOffset + Offset, BytesToMap, Flags) : nullptr;
	}
private:
	IMappedFileHandle* SharedMappedFileHandle;
	const int64 BlockOffset;
};

////////////////////////////////////////////////////////////////////////////////
struct FChunkLookup
{
	const FIoOffsetAndLength* Find(const FIoChunkId& ChunkId) const;

	enum class EType : uint8
	{
		Default,
		Perfect
	};

	struct FPerfectHashMap
	{
		TConstArrayView<int32>				ChunkHashSeeds;
		TConstArrayView<FIoChunkId>			ChunkIds;
		TConstArrayView<FIoOffsetAndLength>	Offsets;
	};

	FPerfectHashMap							PerfectMap;
	TMap<FIoChunkId, FIoOffsetAndLength>	DefaultMap;
	EType									Type = EType::Default;
};

////////////////////////////////////////////////////////////////////////////////
const FIoOffsetAndLength* FChunkLookup::Find(const FIoChunkId& ChunkId) const
{
	if (Type == EType::Default)
	{
		return DefaultMap.Find(ChunkId);
	}

	// See FIoStoreWriterImpl::GeneratePerfectHashes
	const uint32 ChunkCount = PerfectMap.ChunkIds.Num();
	if (ChunkCount == 0)
	{
		return nullptr;
	}

	const uint32 SeedCount	= PerfectMap.ChunkHashSeeds.Num();
	uint32 SeedIndex		= FIoStoreTocResource::HashChunkIdWithSeed(0, ChunkId) % SeedCount;
	const int32 Seed		= PerfectMap.ChunkHashSeeds[SeedIndex];

	if (Seed == 0)
	{
		return nullptr;
	}

	uint32 Slot = MAX_uint32;
	if (Seed < 0)
	{
		const uint32 SeedAsIndex = static_cast<uint32>(-Seed - 1);
		if (SeedAsIndex < ChunkCount)
		{
			Slot = static_cast<uint32>(SeedAsIndex);
		}
		else
		{
			// Entry without perfect hash
			return DefaultMap.Find(ChunkId);
		}
	}
	else
	{
		Slot = FIoStoreTocResource::HashChunkIdWithSeed(static_cast<uint32>(Seed), ChunkId) % ChunkCount;
	}

	if (PerfectMap.ChunkIds[Slot] == ChunkId)
	{
		return &PerfectMap.Offsets[Slot];
	}

	return nullptr;
};
////////////////////////////////////////////////////////////////////////////////
struct FContainer;

struct FContainerPartition
{
	FContainer&						Container;
	FString							Filename;
	FIoFileHandle					FileHandle;
	uint64							FileSize = 0;
	TUniquePtr<IMappedFileHandle>	MappedFileHandle;
};

////////////////////////////////////////////////////////////////////////////////
struct FContainer
{
	static TIoStatusOr<TUniquePtr<FContainer>>			Open(const TCHAR* Filename, int32 MountOrder, uint32 InstanceId, bool bReadsSoftRefs, FIoContainerHeader& OutContainerHeader);
	FContainerPartition&								GetPartition(uint64 Offset, uint64& OutPartitionOffset, int32* OutIndex = nullptr);
	uint64												GetAllocatedSize() const;

	FIoStoreTocResourceStorage							TocStorage;
	FAES::FAESKey										EncryptionKey;
	FChunkLookup										ChunkLookup;
	TArray<FContainerPartition>							Partitions;
	TArray<FName>										CompressionMethods;
	TConstArrayView<FIoStoreTocCompressedBlockEntry>	CompressionBlocks;
	TConstArrayView<FSHAHash>							CompressionBlockHashes;
	FString												BaseFilePath;
	FIoContainerId										ContainerId;
	uint64												PartitionSize = 0;
	uint64												CompressionBlockSize = 0;
	EIoContainerFlags									ContainerFlags;
	int32												MountOrder = MAX_int32;
	uint32												InstanceId = 0;
	std::atomic_int32_t									ActiveReadCount{0};
};

using FUniqueContainer = TUniquePtr<FContainer>;

////////////////////////////////////////////////////////////////////////////////
TIoStatusOr<TUniquePtr<FContainer>>	FContainer::Open(
	const TCHAR* Filename,
	int32 MountOrder,
	uint32 InstanceId,
	bool bReadSoftRefs,
	FIoContainerHeader& OutContainerHeader)
{
	LLM_SCOPE_BYNAME(TEXT("Filesystem/IODispatcher"));
	TRACE_CPUPROFILER_EVENT_SCOPE(ReadContainerHeader);

	FString BaseFilePath = FString(FPathViews::GetBaseFilenameWithPath(Filename));

	TStringBuilder<256> Sb;

	Sb << BaseFilePath << TEXT(".utoc");

	FIoStoreTocResourceView		TocView;
	FIoStoreTocResourceStorage	TocStorage;

	FIoStatus Status = FIoStoreTocResourceView::Read(Sb.ToString(), EIoStoreTocReadOptions::Default, TocView, TocStorage);
	if (Status.IsOk() == false)
	{
		return Status;
	}

	FUniqueContainer Container = MakeUnique<FContainer>();

	if (EnumHasAnyFlags(TocView.Header.ContainerFlags, EIoContainerFlags::Encrypted))
	{
		if (FEncryptionKeyManager::Get().TryGetKey(TocView.Header.EncryptionKeyGuid, Container->EncryptionKey) == false)
		{
			return FIoStatus(EIoErrorCode::InvalidEncryptionKey);
		}
	}

	Container->TocStorage				= MoveTemp(TocStorage);
	Container->BaseFilePath				= BaseFilePath;
	Container->PartitionSize			= TocView.Header.PartitionSize;
	Container->CompressionMethods		= MoveTemp(TocView.CompressionMethods);
	Container->CompressionBlockSize		= TocView.Header.CompressionBlockSize;
	Container->CompressionBlocks		= MoveTemp(TocView.CompressionBlocks);
	Container->CompressionBlockHashes	= MoveTemp(TocView.ChunkBlockSignatures);
	Container->ContainerFlags			= TocView.Header.ContainerFlags;
	Container->ContainerId				= TocView.Header.ContainerId;
	Container->MountOrder				= MountOrder;
	Container->InstanceId				= InstanceId;

	// Parse lookup table information
	if (TocView.ChunkPerfectHashSeeds.IsEmpty() == false)
	{
		for (int32 ChunkIndex : TocView.ChunkIndicesWithoutPerfectHash)
		{
			const FIoChunkId&			ChunkId = TocView.ChunkIds[ChunkIndex];
			const FIoOffsetAndLength&	OffsetLength = TocView.ChunkOffsetLengths[ChunkIndex];
			
			Container->ChunkLookup.DefaultMap.Add(ChunkId, OffsetLength);
		}
		
		Container->ChunkLookup.PerfectMap.ChunkHashSeeds	= MoveTemp(TocView.ChunkPerfectHashSeeds);
		Container->ChunkLookup.PerfectMap.Offsets			= MoveTemp(TocView.ChunkOffsetLengths);
		Container->ChunkLookup.PerfectMap.ChunkIds			= MoveTemp(TocView.ChunkIds);
		Container->ChunkLookup.Type							= FChunkLookup::EType::Perfect;
	}
	else
	{
		for (uint32 ChunkIndex = 0; ChunkIndex < TocView.Header.TocEntryCount; ++ChunkIndex)
		{
			const FIoChunkId&			ChunkId = TocView.ChunkIds[ChunkIndex];
			const FIoOffsetAndLength&	OffsetLength = TocView.ChunkOffsetLengths[ChunkIndex];

			Container->ChunkLookup.DefaultMap.Add(ChunkId, OffsetLength);
		}
		Container->ChunkLookup.Type	= FChunkLookup::EType::Default;
	}

	// Open partition file handles 
	Container->Partitions.Reserve(TocView.Header.PartitionCount);
	for (uint32 PartitionIndex = 0; PartitionIndex < TocView.Header.PartitionCount; PartitionIndex++)
	{
		FContainerPartition& Part = Container->Partitions.Add_GetRef(FContainerPartition
		{
			.Container	= *Container,
		});

		Sb.Reset();
		Sb << BaseFilePath;
		if (PartitionIndex > 0)
		{
			Sb.Appendf(TEXT("_s%d"), PartitionIndex);
		}
		Sb.Append(TEXT(".ucas"));

		EIoFilePropertyFlags FileFlags = EIoFilePropertyFlags::None;
		if (EnumHasAnyFlags(Container->ContainerFlags, EIoContainerFlags::Encrypted))
		{
			FileFlags |= EIoFilePropertyFlags::Encrypted;
		}
		if (EnumHasAnyFlags(Container->ContainerFlags, EIoContainerFlags::Signed))
		{
			FileFlags |= EIoFilePropertyFlags::Signed;
		}

		FIoFileProperties FileProperties
		{
			.CompressionMethods		= Container->CompressionMethods,
			.CompressionBlockSize	= IntCastChecked<uint32>(Container->CompressionBlockSize),
			.Flags					= FileFlags
		};
		
		FIoFileStat FileStats;
		TIoStatusOr<FIoFileHandle> Handle = FPlatformIoDispatcher::Get().OpenFile(
			Sb.ToString(),
			FileProperties,
			&FileStats);

		if (Handle.IsOk() == false)
		{
			return Handle.Status();
		}

		Part.Filename	= Sb.ToString();
		Part.FileHandle = Handle.ConsumeValueOrDie();
		Part.FileSize	= FileStats.FileSize;
	}

	// Read the container header information 
	const FIoChunkId HeaderChunkId				= CreateContainerHeaderChunkId(Container->ContainerId);
	const FIoOffsetAndLength* OffsetAndLength	= Container->ChunkLookup.Find(HeaderChunkId);

	// Deserialize the container header
	if (OffsetAndLength != nullptr)
	{
		const uint32 FirstBlock		= uint32(OffsetAndLength->GetOffset() / Container->CompressionBlockSize);
		const uint32 LastBlock		= uint32((OffsetAndLength->GetOffset() + OffsetAndLength->GetLength() - 1) / Container->CompressionBlockSize);
		uint64 EncodedSize			= 0;

		FName CompressionMethod = NAME_None;
		TArray<uint32> BlockSizes;
		BlockSizes.Reserve((LastBlock - FirstBlock) + 1);

		for (uint32 Idx = FirstBlock; Idx <= LastBlock; ++Idx)
		{
			const FIoStoreTocCompressedBlockEntry& Block	= Container->CompressionBlocks[Idx];
			const FName BlockCompressionMethod				= Container->CompressionMethods[Block.GetCompressionMethodIndex()];
			const uint32 EncodedBlockSize					= Block.GetCompressedSize();

			if (BlockCompressionMethod != NAME_None)
			{
				ensure(CompressionMethod == NAME_None || CompressionMethod == BlockCompressionMethod);
				CompressionMethod = BlockCompressionMethod;
			}

			BlockSizes.Add(EncodedBlockSize);
			EncodedSize += Align(EncodedBlockSize, FAES::AESBlockSize); // Size on disk is always aligned to AES block size
		}

		uint64 PartitionOffset	= MAX_uint64;
		int32 PartitionIndex	= INDEX_NONE;

		FContainerPartition& Partition = Container->GetPartition(
			Container->CompressionBlocks[FirstBlock].GetOffset(),
			PartitionOffset,
			&PartitionIndex);

		IPlatformFile& Ipf = FPlatformFileManager::Get().GetPlatformFile();
		TUniquePtr<IFileHandle> FileHandle(Ipf.OpenRead(*Partition.Filename));
		if (FileHandle.IsValid() == false)
		{
			return FIoStatus(FIoStatusBuilder(EIoErrorCode::FileOpenFailed)
				<< TEXT("Failed to open container '") << Sb.ToString() << TEXT("'"));
		}

		if (FileHandle->Seek(PartitionOffset) == false)
		{
			return FIoStatus(FIoStatusBuilder(EIoErrorCode::FileOpenFailed)
				<< TEXT("Failed to seek to container header offset"));
		}

		FIoBuffer EncodedBlocks(EncodedSize);
		if (FileHandle->Read(EncodedBlocks.GetData(), EncodedBlocks.GetSize()) == false)
		{
			return FIoStatus(FIoStatusBuilder(EIoErrorCode::FileOpenFailed)
				<< TEXT("Failed to read container header chunk"));
		}

		const FMemoryView EncryptionKey = EnumHasAnyFlags(Container->ContainerFlags, EIoContainerFlags::Encrypted)
			? MakeMemoryView(Container->EncryptionKey.Key, FAES::FAESKey::KeySize)
			: FMemoryView();

		FIoBuffer DecodedChunk(OffsetAndLength->GetLength());

		FIoChunkDecodingParams Params;
		Params.CompressionFormat	= CompressionMethod; 
		Params.EncryptionKey		= EncryptionKey; 
		Params.BlockSize			= uint32(Container->CompressionBlockSize);
		Params.TotalRawSize			= OffsetAndLength->GetLength();
		Params.EncodedBlockSize		= BlockSizes;

		if (FIoChunkEncoding::Decode(Params, EncodedBlocks.GetView(), DecodedChunk.GetMutableView()) == false)
		{
			return FIoStatus(FIoStatusBuilder(EIoErrorCode::FileOpenFailed)
				<< TEXT("Failed to deserialize container header"));
		}

		FMemoryReaderView Ar(DecodedChunk.GetView());
		Ar << OutContainerHeader;

		if (Ar.IsError() || Ar.IsCriticalError())
		{
			return FIoStatus(FIoStatusBuilder(EIoErrorCode::FileOpenFailed)
				<< TEXT("Failed to serialize container header"));
		}

		if (bReadSoftRefs)
		{
			if (OutContainerHeader.SoftPackageReferencesSerialInfo.Offset < 0)
			{
				return FIoStatusBuilder(EIoErrorCode::ReadError)
					<< FString::Printf(TEXT("Invalid soft package reference offset '%" INT64_FMT "'"), OutContainerHeader.SoftPackageReferencesSerialInfo.Offset);
			}

			if ((OutContainerHeader.SoftPackageReferencesSerialInfo.Offset + OutContainerHeader.SoftPackageReferencesSerialInfo.Size) > Ar.TotalSize())
			{
				return FIoStatusBuilder(EIoErrorCode::ReadError)
					<< FString::Printf(TEXT("Soft package reference offset '%" INT64_FMT "' and size '%" INT64_FMT "' will seek past the end of archive size '%" INT64_FMT "'"),
						OutContainerHeader.SoftPackageReferencesSerialInfo.Offset,
						OutContainerHeader.SoftPackageReferencesSerialInfo.Size,
						Ar.TotalSize());
			}

			Ar.Seek(OutContainerHeader.SoftPackageReferencesSerialInfo.Offset);
			Ar << OutContainerHeader.SoftPackageReferences;
		}
	}

	return Container;
}

FContainerPartition& FContainer::GetPartition(uint64 Offset, uint64& OutPartitionOffset, int32* OutIndex)
{
	const int32 PartitionIndex	= IntCastChecked<int32>(Offset / PartitionSize);
	OutPartitionOffset			= Offset % PartitionSize;

	if (OutIndex != nullptr)
	{
		*OutIndex = PartitionIndex;
	}

	ensure(PartitionIndex < Partitions.Num());
	return Partitions[PartitionIndex];
}

uint64 FContainer::GetAllocatedSize() const
{
	return TocStorage.GetAllocatedSize() + ChunkLookup.DefaultMap.GetAllocatedSize();
}

////////////////////////////////////////////////////////////////////////////////
struct FChunkInfo
{
				FChunkInfo() = default;
				FChunkInfo(FContainer* Container, const FIoOffsetAndLength* OffsetLength);
	bool		IsValid() const { return Container != nullptr; }
	uint64		Offset() const { return OffsetLength->GetOffset(); }
	uint64		Size() const { return OffsetLength->GetLength(); }
	FContainer& GetContainer() { return *Container; }

private:
	FContainer*					Container = nullptr;
	const FIoOffsetAndLength*	OffsetLength = nullptr;
};

////////////////////////////////////////////////////////////////////////////////
FChunkInfo::FChunkInfo(FContainer* InContainer, const FIoOffsetAndLength* InOffsetLength)
	: Container(InContainer)
	, OffsetLength(InOffsetLength)
{
}

////////////////////////////////////////////////////////////////////////////////
class FFileIoStore
{
public:
	FChunkInfo						GetChunkInfo(const FIoChunkId& ChunkId) const;
	TIoStatusOr<FIoContainerHeader>	Mount(const TCHAR* TocPath, int32 MountOrder, ETocMountOptions Options);
	bool							Unmount(const TCHAR* TocPath);
	void							ReopenAllFileHandles();
	FRWLock&						GetLock() { return RwLock; }
	FRWLock&						GetLock() const { return RwLock; }

private:
	TArray<FUniqueContainer>		MountedContainers;
	mutable FRWLock					RwLock;
	std::atomic_uint32_t			ContainerInstanceId{1};
};

FChunkInfo FFileIoStore::GetChunkInfo(const FIoChunkId& ChunkId) const
{
	for (const FUniqueContainer& Container : MountedContainers)
	{
		if (const FIoOffsetAndLength* OffsetLength = Container->ChunkLookup.Find(ChunkId))
		{
			return FChunkInfo(Container.Get(), OffsetLength);
		}
	}

	return FChunkInfo();
}

////////////////////////////////////////////////////////////////////////////////
TIoStatusOr<FIoContainerHeader>	FFileIoStore::Mount(const TCHAR* TocPath, int32 MountOrder, ETocMountOptions Options)
{
	const uint32		InstanceId = ContainerInstanceId.fetch_add(1, std::memory_order_relaxed);
	FIoContainerHeader	Hdr;

	const bool bReadSoftRefs = EnumHasAnyFlags(Options, ETocMountOptions::WithSoftReferences);
	TIoStatusOr<FUniqueContainer> Status = FContainer::Open(TocPath, MountOrder, InstanceId, bReadSoftRefs, Hdr);
	if (Status.IsOk() == false)
	{
		return Status.Status();
	}

	FUniqueContainer Container				= Status.ConsumeValueOrDie();
	const EIoContainerFlags ContainerFlags	= Container->ContainerFlags;
	int32 MountIndex						= INDEX_NONE;
	{
		FWriteScopeLock _(RwLock);

		MountIndex = Algo::UpperBound(
			MountedContainers,
			Container,
			[](const FUniqueContainer& A, const FUniqueContainer& B)
			{
				if (A->MountOrder != B->MountOrder)
				{
					return A->MountOrder > B->MountOrder;
				}
				return A->InstanceId > B->InstanceId;
			});

		MountedContainers.Insert(MoveTemp(Container), MountIndex);
	}

	UE_LOGF(LogIoStore, Display, "Mounted container '%ls', Id='%ls', Order=%d, Slot=%d, Options=(%ls), Flags=(%ls), HasSoftRefs=%ls",
		TocPath, *LexToString(Hdr.ContainerId), MountOrder, MountIndex, LexToString(Options), *LexToString(ContainerFlags),
		Hdr.SoftPackageReferences.bContainsSoftPackageReferences ? TEXT("True") : TEXT("False"));

	return Hdr;
}

bool FFileIoStore::Unmount(const TCHAR* TocPath)
{
	const FString BaseFilePath = FString(FPathViews::GetBaseFilenameWithPath(TocPath));
	
	FUniqueContainer ContainerToRemove;
	{
		FWriteScopeLock _(RwLock);

		int32 ContainerIdx = INDEX_NONE;
		for (int32 Idx = 0; FUniqueContainer& Container : MountedContainers)
		{
			if (Container->BaseFilePath == BaseFilePath)
			{
				ContainerIdx = Idx;
				break;
			}
			++Idx;
		}

		if (ContainerIdx != INDEX_NONE)
		{
			ContainerToRemove = MoveTemp(MountedContainers[ContainerIdx]);
			MountedContainers.RemoveAt(ContainerIdx);
		}
	}

	if (ContainerToRemove.IsValid() == false)
	{
		UE_LOGF(LogIoStore, Warning, "Failed to unmount container '%ls', reason 'Not Found'", TocPath);
		return false;
	}

	if (ContainerToRemove->ActiveReadCount.load(std::memory_order_seq_cst) > 0)
	{
		for (FContainerPartition& Part : ContainerToRemove->Partitions)
		{
			UE_LOGF(LogIoStore, Log, "Cancelling inflight read requests for file '%ls'", *Part.Filename);
			UE::FPlatformIoDispatcher::Get().CancelAllRequests((Part.FileHandle));
		}

		UE_LOGF(LogIoStore, Log, "Waiting for read request(s) to finish before unmounting '%ls.utoc'", TocPath);
		const double MaxWaitTimeSeconds	= Config::FileIoStoreUnmountTimeOutSeconds;
		const double StartTime			= FMath::Clamp(FPlatformTime::Seconds(), 5.0f, 30.0f);
		while (ContainerToRemove->ActiveReadCount.load(std::memory_order_seq_cst) > 0)
		{
			FPlatformProcess::Sleep(0);
			if (FPlatformTime::Seconds() - StartTime > MaxWaitTimeSeconds)
			{
				UE_LOGF(LogIoStore, Warning, "Stopped waiting for read request(s) after %.2lf seconds", MaxWaitTimeSeconds);
				break;
			}
		}
	}

	for (FContainerPartition& Part : ContainerToRemove->Partitions)
	{
		UE::FPlatformIoDispatcher::Get().CloseFile((Part.FileHandle));
	}

	UE_LOGF(LogIoStore, Log, "Unmounted container '%ls'", TocPath);
	return true;
}

void FFileIoStore::ReopenAllFileHandles()
{
	FWriteScopeLock _(RwLock);
	for (FUniqueContainer& Container : MountedContainers)
	{
		UE_CLOGF(Container->ActiveReadCount.load(std::memory_order_seq_cst) > 0, LogIoStore, Warning, "Calling ReopenAllFileHandles with read requests in flight");
		for (FContainerPartition& Part : Container->Partitions)
		{
			UE_LOGF(LogIoStore, Log, "Reopening container file '%ls'", *Part.Filename);
			UE::FPlatformIoDispatcher::Get().CloseFile((Part.FileHandle));

			EIoFilePropertyFlags FileFlags = EIoFilePropertyFlags::None;
			if (EnumHasAnyFlags(Container->ContainerFlags, EIoContainerFlags::Encrypted))
			{
				FileFlags |= EIoFilePropertyFlags::Encrypted;
			}
			if (EnumHasAnyFlags(Container->ContainerFlags, EIoContainerFlags::Signed))
			{
				FileFlags |= EIoFilePropertyFlags::Signed;
			}

			FIoFileProperties FileProperties
			{
				.CompressionMethods		= Container->CompressionMethods,
				.CompressionBlockSize	= IntCastChecked<uint32>(Container->CompressionBlockSize),
				.Flags					= FileFlags
			};

			TIoStatusOr<FIoFileHandle> Handle = FPlatformIoDispatcher::Get().OpenFile(*Part.Filename, FileProperties);
			Part.FileHandle = Handle.ConsumeValueOrDie();
		}
	}
}

////////////////////////////////////////////////////////////////////////////////
struct FResolvedRequest
{
	explicit FResolvedRequest(FIoRequestImpl& InDispatcherRequest)
		: DispatcherRequest(InDispatcherRequest)
	{
		check(DispatcherRequest.BackendData == nullptr);
		DispatcherRequest.BackendData = this;
	}

	static FResolvedRequest& Get(FIoRequestImpl& DispatcherRequest)
	{
		return *reinterpret_cast<FResolvedRequest*>(DispatcherRequest.BackendData);
	}

	static FResolvedRequest* TryGet(FIoRequestImpl* DispatcherRequest)
	{
		if (DispatcherRequest != nullptr && DispatcherRequest->BackendData != nullptr)
		{
			return reinterpret_cast<FResolvedRequest*>(DispatcherRequest->BackendData);
		}
		return nullptr;
	}

	FIoRequestImpl&		DispatcherRequest;
	FIoBuffer			Buffer;
	FChunkInfo			ChunkInfo;
	FIoFileReadRequest	PlatformRequest;
	FIoFileHandle		FileHandle;
	uint64				Offset = MAX_uint64;
	uint64				Size = MAX_uint64;
};

using FRequestAllocator = TSingleThreadedSlabAllocator<FResolvedRequest>;

////////////////////////////////////////////////////////////////////////////////
class FFileIoDispatcherBackend final
	: public IFileIoDispatcherBackend
{
	using FSharedBackendContext = TSharedPtr<const FIoDispatcherBackendContext>;
public:
											FFileIoDispatcherBackend();
	virtual									~FFileIoDispatcherBackend();

	// IFileIoDispatcherBackend
	virtual TIoStatusOr<FIoContainerHeader> Mount(
		const TCHAR* TocPath, 
		int32 Order, 
		const FGuid& EncryptionKeyGuid, 
		const FAES::FAESKey& EncryptionKey, 
		ETocMountOptions Options) override;
	virtual bool							Unmount(const TCHAR* TocPath) override;
	virtual void							ReopenAllFileHandles() override;

	// IIoDispatcherBackend
	virtual void							Initialize(TSharedRef<const FIoDispatcherBackendContext> Context) override;
	virtual void							Shutdown() override;
	virtual void							ResolveIoRequests(FIoRequestList Requests, FIoRequestList& OutUnresolved) override;
	virtual FIoRequestImpl* 				GetCompletedIoRequests() override;
	virtual void							CancelIoRequest(FIoRequestImpl* Request) override;
	virtual void							UpdatePriorityForIoRequest(FIoRequestImpl* Request) override;
	virtual bool							DoesChunkExist(const FIoChunkId& ChunkId) const override;
	virtual TIoStatusOr<uint64>				GetSizeForChunk(const FIoChunkId& ChunkId) const override;
	virtual TIoStatusOr<FIoMappedRegion>	OpenMapped(const FIoChunkId& ChunkId, const FIoReadOptions& Options) override;
	virtual const TCHAR*					GetName() const { return TEXT("File"); }

private:
	void									HandleSignatureError(FIoRequestImpl& DispatcherRequest, uint32 FailedBlockIndex);

	FSharedBackendContext		BackendContext;
	FFileIoStore				IoStore;
	FRequestAllocator			RequestAllocator;
	FIoRequestList				CompletedDispatcherRequests;
	UE::FMutex					Mutex;
};

////////////////////////////////////////////////////////////////////////////////
FFileIoDispatcherBackend::FFileIoDispatcherBackend()
{
}

FFileIoDispatcherBackend::~FFileIoDispatcherBackend()
{
}

TIoStatusOr<FIoContainerHeader> FFileIoDispatcherBackend::Mount(
	const TCHAR* TocPath, 
	int32 Order, 
	const FGuid& EncryptionKeyGuid, 
	const FAES::FAESKey& EncryptionKey, 
	ETocMountOptions Options)
{
	LLM_SCOPE_BYNAME(TEXT("FileSystem/FileIoStore"));
	return IoStore.Mount(TocPath, Order, Options);
}

bool FFileIoDispatcherBackend::Unmount(const TCHAR* TocPath)
{
	return IoStore.Unmount(TocPath);
}

void FFileIoDispatcherBackend::ReopenAllFileHandles()
{
	IoStore.ReopenAllFileHandles();
}

void FFileIoDispatcherBackend::Initialize(TSharedRef<const FIoDispatcherBackendContext> Context)
{
	BackendContext = Context;
}

void FFileIoDispatcherBackend::Shutdown()
{
}

void FFileIoDispatcherBackend::ResolveIoRequests(FIoRequestList Requests, FIoRequestList& OutUnresolved)
{
	FIoRequestList ResolvedRequests;
	{
		FReadScopeLock ReadScope(IoStore.GetLock());

		while (FIoRequestImpl* DispatcherRequest = Requests.PopHead())
		{
			const FChunkInfo ChunkInfo = IoStore.GetChunkInfo(DispatcherRequest->ChunkId);
			if (ChunkInfo.IsValid() == false)
			{
				OutUnresolved.AddTail(DispatcherRequest);
				continue;
			}

			const int64 ResolvedSize = FMath::Min(DispatcherRequest->Options.GetSize(), ChunkInfo.Size() - DispatcherRequest->Options.GetOffset());
			if (ResolvedSize > 0)
			{
				FResolvedRequest* ResolvedRequest	= RequestAllocator.Construct(*DispatcherRequest);
				ResolvedRequest->ChunkInfo			= ChunkInfo;
				ResolvedRequest->Offset				= ChunkInfo.Offset() + DispatcherRequest->Options.GetOffset();
				ResolvedRequest->Size				= ResolvedSize; 
				check(DispatcherRequest->BackendData != nullptr);

				ResolvedRequests.AddTail(DispatcherRequest);

				if (DispatcherRequest->Options.GetTargetVa() != nullptr)
				{
					DispatcherRequest->CreateBuffer(ResolvedRequest->Size);
				}
			}
			else
			{
				if (ResolvedSize < 0)
				{
					DispatcherRequest->SetFailed();
				}
				else
				{
					DispatcherRequest->CreateBuffer(0);
				}
				TUniqueLock Lock(Mutex);
				CompletedDispatcherRequests.AddTail(DispatcherRequest);
			}
		}
	}

	while (FIoRequestImpl* DispatcherRequest = ResolvedRequests.PopHead())
	{
		FResolvedRequest& ResolvedRequest = FResolvedRequest::Get(*DispatcherRequest);

		FIoBuffer& Dst						= DispatcherRequest->HasBuffer() ? DispatcherRequest->GetBuffer() : ResolvedRequest.Buffer;
		FContainer& Container				= ResolvedRequest.ChunkInfo.GetContainer();
		const int32 FirstCompressedBlock	= IntCastChecked<int32>(ResolvedRequest.Offset / Container.CompressionBlockSize);
		const int32 LastCompressedBlock		= IntCastChecked<int32>((ResolvedRequest.Offset + ResolvedRequest.Size - 1) / Container.CompressionBlockSize);
		uint64 RequestStartOffsetInBlock	= ResolvedRequest.Offset - (FirstCompressedBlock * Container.CompressionBlockSize);

		// All encoded blocks for a chunk always resides in the same .ucas file 
		const FIoStoreTocCompressedBlockEntry& FirstBlock	= Container.CompressionBlocks[FirstCompressedBlock];
		uint64 FirstBlockOffsetInPartition					= MAX_uint64;
		FContainerPartition& Partition						= Container.GetPartition(FirstBlock.GetOffset(), FirstBlockOffsetInPartition);
		ResolvedRequest.FileHandle							= Partition.FileHandle;

		// On some platforms we can read directly into the destination buffer
		{
			Container.ActiveReadCount.fetch_add(1, std::memory_order_relaxed);
			ResolvedRequest.PlatformRequest = FPlatformIoDispatcher::Get().ReadDirect(
				FIoDirectReadRequestParams
				{
					.FileHandle = Partition.FileHandle,
					.Dst		= Dst,
					.Offset		= FirstBlockOffsetInPartition + RequestStartOffsetInBlock,
					.Size		= ResolvedRequest.Size,
					.UserData	= DispatcherRequest
				},
				[this](FIoFileReadResult&& Result)
				{
					FIoRequestImpl* DispatcherRequest = reinterpret_cast<FIoRequestImpl*>(Result.UserData);
					if (Result.ErrorCode != EIoErrorCode::Ok)
					{
						DispatcherRequest->SetFailed();
					}

					{
						UE::TUniqueLock Lock(Mutex);
						CompletedDispatcherRequests.AddTail(DispatcherRequest);
					}
					BackendContext->WakeUpDispatcherThreadDelegate.Execute();
				});

			if (ResolvedRequest.PlatformRequest.IsValid())
			{
				continue;
			}
			else
			{
				Container.ActiveReadCount.fetch_sub(1, std::memory_order_relaxed);
			}
		}

		FIoScatterGatherRequestParams ScatterGather(Partition.FileHandle, Dst, ResolvedRequest.Size, DispatcherRequest, DispatcherRequest->Priority);

		// Scatter offsets
		uint64 RequestRemainingBytes	= ResolvedRequest.Size;
		uint64 OffsetInRequest			= 0;
		uint64 BlockFileOffset			= FirstBlockOffsetInPartition;		

		for (int32 BlockIndex = FirstCompressedBlock; BlockIndex <= LastCompressedBlock; ++BlockIndex)
		{
			const FIoStoreTocCompressedBlockEntry& CompressedBlock	= Container.CompressionBlocks[BlockIndex];

			const uint32 BlockCompressedSize	= CompressedBlock.GetCompressedSize();
			const uint32 BlockUncompressedSize	= CompressedBlock.GetUncompressedSize();
			const uint32 BlockFileSize			= Align(BlockCompressedSize, FAES::AESBlockSize);
			const uint64 ScatterOffset			= RequestStartOffsetInBlock;
			const uint64 ScatterSize			= FMath::Min<uint64>(CompressedBlock.GetUncompressedSize() - RequestStartOffsetInBlock, RequestRemainingBytes);
			const uint64 DstOffset				= OffsetInRequest;

			FMemoryView EncryptionKey;
			if (EnumHasAnyFlags(Container.ContainerFlags, EIoContainerFlags::Encrypted))
			{
				EncryptionKey = MakeMemoryView(Container.EncryptionKey.Key, FAES::FAESKey::KeySize);
			}

			FName CompressionMethod = NAME_None;
			if (EnumHasAnyFlags(Container.ContainerFlags, EIoContainerFlags::Compressed))
			{
				CompressionMethod = Container.CompressionMethods[CompressedBlock.GetCompressionMethodIndex()];
			}

			FMemoryView BlockHash;
			if (EnumHasAnyFlags(Container.ContainerFlags, EIoContainerFlags::Signed))
			{
				const FSHAHash& ShaHash = Container.CompressionBlockHashes[BlockIndex];
				BlockHash				= MakeMemoryView(ShaHash.Hash, sizeof(ShaHash.Hash));
			}

			ScatterGather.Scatter(
				BlockFileOffset, 
				BlockIndex,
				BlockCompressedSize,
				BlockUncompressedSize,
				ScatterOffset,
				ScatterSize,
				DstOffset,
				CompressionMethod,
				EncryptionKey,
				BlockHash);

			BlockFileOffset				+= BlockFileSize;
			RequestRemainingBytes		-= ScatterSize; 
			OffsetInRequest				+= ScatterSize;
			RequestStartOffsetInBlock	= 0;
		}

		Container.ActiveReadCount.fetch_add(1, std::memory_order_relaxed);
		ResolvedRequest.PlatformRequest = FPlatformIoDispatcher::Get().ScatterGather(
			MoveTemp(ScatterGather),
			[this](FIoFileReadResult&& Result)
			{
				FIoRequestImpl* DispatcherRequest = reinterpret_cast<FIoRequestImpl*>(Result.UserData);
				if (DispatcherRequest->IsCancelled() == false && Result.ErrorCode != EIoErrorCode::Ok) 
				{
					DispatcherRequest->SetFailed();
					if (Result.ErrorCode == EIoErrorCode::SignatureError)
					{
						HandleSignatureError(*DispatcherRequest, Result.FailedBlockId);
					}
				}
				{
					UE::TUniqueLock Lock(Mutex);
					CompletedDispatcherRequests.AddTail(DispatcherRequest);
				}
				BackendContext->WakeUpDispatcherThreadDelegate.Execute();
			});

		if (ResolvedRequest.PlatformRequest.IsValid() == false)
		{
			UE_LOGF(LogIoStore, Warning, "Failed to create platform read request, ChunkId='%ls' Filenname='%ls'",
				*LexToString(ResolvedRequest.DispatcherRequest.ChunkId), *Container.BaseFilePath);
			ResolvedRequest.DispatcherRequest.SetFailed();	
			{
				UE::TUniqueLock Lock(Mutex);
				CompletedDispatcherRequests.AddTail(DispatcherRequest);
			}
		}
	}
}

FIoRequestImpl* FFileIoDispatcherBackend::GetCompletedIoRequests()
{
	LLM_SCOPE_BYNAME(TEXT("FileSystem/FileIoStore"));

	FIoRequestList LocalCompletedDispatcherRequests;
	{
		UE::TUniqueLock Lock(Mutex);
		LocalCompletedDispatcherRequests = MoveTemp(CompletedDispatcherRequests);
		CompletedDispatcherRequests = FIoRequestList();
	}

	for (FIoRequestImpl& DispatcherRequest : LocalCompletedDispatcherRequests)
	{
		FResolvedRequest& ResolvedRequest = FResolvedRequest::Get(DispatcherRequest);
		FPlatformIoDispatcher::Get().DeleteRequest(ResolvedRequest.PlatformRequest);
		check(ResolvedRequest.ChunkInfo.GetContainer().ActiveReadCount > 0);
		ResolvedRequest.ChunkInfo.GetContainer().ActiveReadCount.fetch_sub(1, std::memory_order_relaxed);

		const bool bSucceeded = !DispatcherRequest.IsFailed() && !DispatcherRequest.IsCancelled();
		check(!bSucceeded || ResolvedRequest.Buffer.GetSize() > 0 || DispatcherRequest.GetBuffer().GetSize() > 0);
		if (bSucceeded)
		{
			if (DispatcherRequest.HasBuffer() == false)
			{
				DispatcherRequest.SetResult(MoveTemp(ResolvedRequest.Buffer));
			}
		}

		RequestAllocator.Destroy(&ResolvedRequest);
		DispatcherRequest.BackendData = nullptr;
	}

	return LocalCompletedDispatcherRequests.GetHead();
}

void FFileIoDispatcherBackend::CancelIoRequest(FIoRequestImpl* DispatcherRequest)
{
	if (FResolvedRequest* ResolvedRequest = FResolvedRequest::TryGet(DispatcherRequest))
	{
		FPlatformIoDispatcher::Get().CancelRequest(ResolvedRequest->PlatformRequest);
	}
}

void FFileIoDispatcherBackend::UpdatePriorityForIoRequest(FIoRequestImpl* DispatcherRequest)
{
	if (FResolvedRequest* ResolvedRequest = FResolvedRequest::TryGet(DispatcherRequest))
	{
		FPlatformIoDispatcher::Get().UpdatePriority(ResolvedRequest->PlatformRequest, DispatcherRequest->Priority);
	}
}

bool FFileIoDispatcherBackend::DoesChunkExist(const FIoChunkId& ChunkId) const
{
	FReadScopeLock ReadScope(IoStore.GetLock());

	const FChunkInfo ChunkInfo = IoStore.GetChunkInfo(ChunkId);
	return ChunkInfo.IsValid();
}

TIoStatusOr<uint64> FFileIoDispatcherBackend::GetSizeForChunk(const FIoChunkId& ChunkId) const
{
	FReadScopeLock ReadScope(IoStore.GetLock());

	if (const FChunkInfo ChunkInfo = IoStore.GetChunkInfo(ChunkId); ChunkInfo.IsValid())
	{
		return ChunkInfo.Size();
	}

	return FIoStatus::Unknown;
}

TIoStatusOr<FIoMappedRegion> FFileIoDispatcherBackend::OpenMapped(const FIoChunkId& ChunkId, const FIoReadOptions& Options)
{
	if (!FPlatformProperties::SupportsMemoryMappedFiles())
	{
		return FIoStatus(EIoErrorCode::Unknown, TEXT("Platform does not support memory mapped files"));
	}

	if (Options.GetTargetVa() != nullptr)
	{
		return FIoStatus(EIoErrorCode::InvalidParameter, TEXT("Invalid read options"));
	}

	FWriteScopeLock WriteScope(IoStore.GetLock()); // In case a new mapped file handle is created
	FChunkInfo ChunkInfo = IoStore.GetChunkInfo(ChunkId);
	if (ChunkInfo.IsValid() == false)
	{
		return FIoStatus(EIoErrorCode::NotFound);
	}

	const int64 ResolvedOffset	= ChunkInfo.Offset();
	const int64 ResolvedSize	= FMath::Min(Options.GetSize(), ChunkInfo.Size() - Options.GetOffset());
	FContainer& Container		= ChunkInfo.GetContainer();
	const int32 BlockIndex		= IntCastChecked<int32>(ResolvedOffset / Container.CompressionBlockSize);

	const FIoStoreTocCompressedBlockEntry& Block	= Container.CompressionBlocks[BlockIndex];
	uint64 BlockOffsetInPartition					= MAX_uint64;
	FContainerPartition& Partition					= Container.GetPartition(Block.GetOffset(), BlockOffsetInPartition);

	if (Partition.MappedFileHandle.IsValid() == false)
	{
		IPlatformFile& Ipf = FPlatformFileManager::Get().GetPlatformFile();
		if (FOpenMappedResult Result = Ipf.OpenMappedEx2(*Partition.Filename); !Result.HasError())
		{
			Partition.MappedFileHandle = Result.StealValue();
		}
		else
		{
			UE_LOGF(LogIoStore, Error, "Failed to memory map chunk, ChunkId='%ls', Filename='%ls', Reason='%ls'",
				*LexToString(ChunkId), *Partition.Filename, *Result.GetError().GetMessage());
		}
	}

	if (Partition.MappedFileHandle.IsValid() == false)
	{
		return FIoStatus(EIoErrorCode::FileOpenFailed);
	}

	IMappedFileHandle& MappedFileHandle = *Partition.MappedFileHandle.Get(); 
	if (IMappedFileRegion* MappedFileRegion = MappedFileHandle.MapRegion(BlockOffsetInPartition + Options.GetOffset(), ResolvedSize))
	{
		return FIoMappedRegion
		{
			.MappedFileHandle	= new FMappedFileProxy(Partition.MappedFileHandle.Get(), MappedFileHandle.GetFileSize(), IntCastChecked<int64>(BlockOffsetInPartition)),
			.MappedFileRegion	= MappedFileRegion
		};
	}
	else
	{
		UE_LOGF(LogIoStore, Error, "Failed to memory map region, ChunkId='%ls', Filename='%ls', Region=(%llu, %lld)",
			*LexToString(ChunkId), *Partition.Filename, BlockOffsetInPartition + Options.GetOffset(), ResolvedSize);
	}

	return FIoStatus(EIoErrorCode::FileOpenFailed);
}

void FFileIoDispatcherBackend::HandleSignatureError(FIoRequestImpl& DispatcherRequest, uint32 FailedBlockIndex)
{
	FIoSignatureError SignatureError;
	{
		FWriteScopeLock _(IoStore.GetLock());

		FResolvedRequest& ResolvedRequest				= FResolvedRequest::Get(DispatcherRequest);
		const FContainer& Container						= ResolvedRequest.ChunkInfo.GetContainer();
		const FIoStoreTocCompressedBlockEntry& Block	= Container.CompressionBlocks[FailedBlockIndex];

		int32 PartIdx = 0; 
		for (const FContainerPartition& Part : Container.Partitions)
		{
			if (Part.FileHandle.Value() == ResolvedRequest.FileHandle.Value())
			{
				break;
			}
			PartIdx++;
		}

		UE_LOGF(LogIoStore, Warning, "Signature error detected, ChunkId='%ls', Filename='%ls', Offset=%llu",
				*LexToString(DispatcherRequest.ChunkId), *Container.Partitions[PartIdx].Filename, Block.GetOffset());

		SignatureError.ContainerName	= Container.BaseFilePath; 
		SignatureError.BlockIndex		= FailedBlockIndex; 
		SignatureError.ExpectedHash		= Container.CompressionBlockHashes[FailedBlockIndex];
		//SignatureError.ActualHash		= Is this really needed?
	}

	check(BackendContext);
	if (BackendContext->SignatureErrorDelegate.IsBound())
	{
		BackendContext->SignatureErrorDelegate.Broadcast(SignatureError);
	}
}

////////////////////////////////////////////////////////////////////////////////
TSharedPtr<IFileIoDispatcherBackend> MakeFileIoDispatcherBackend()
{
	return MakeShared<FFileIoDispatcherBackend>();
}

} // namespace UE::IoStore
