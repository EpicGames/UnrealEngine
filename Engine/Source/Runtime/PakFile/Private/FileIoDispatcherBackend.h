// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IO/IoDispatcherBackend.h"
#include "IO/IoStatus.h"
#include "Misc/AES.h"
#include "Misc/EnumClassFlags.h"

struct FIoContainerHeader;

namespace UE::IoStore
{

enum class ETocMountOptions
{
	None				= 0,
	/** Make soft references available */
	WithSoftReferences	= 1 << 0,
	Last				= WithSoftReferences
};
ENUM_CLASS_FLAGS(ETocMountOptions);

class IFileIoDispatcherBackend
	: public IIoDispatcherBackend
{
public:
	virtual ~IFileIoDispatcherBackend() = default;

	virtual TIoStatusOr<FIoContainerHeader> Mount(
		const TCHAR* TocPath, 
		int32 Order, 
		const FGuid& EncryptionKeyGuid, 
		const FAES::FAESKey& EncryptionKey, 
		ETocMountOptions Options = ETocMountOptions::None) = 0;
	virtual bool							Unmount(const TCHAR* TocPath) = 0;
	virtual void							ReopenAllFileHandles() = 0;
};

TSharedPtr<IFileIoDispatcherBackend> MakeFileIoDispatcherBackend();

} // namespace UE::IoStore

inline const TCHAR* LexToString(UE::IoStore::ETocMountOptions MountOptions)
{
	static const TCHAR* Names[]
	{
		TEXT("None"),
		TEXT("WithSoftReferences")
	};
	static_assert(std::size(Names) == (1 + SIZE_T(UE::IoStore::ETocMountOptions::Last)));

	// Update this if more options are added
	return Names[int32(MountOptions)];
}
