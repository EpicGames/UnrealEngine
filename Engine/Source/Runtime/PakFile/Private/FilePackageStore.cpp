// Copyright Epic Games, Inc. All Rights Reserved.
#include "FilePackageStore.h"
#include "Algo/Find.h"
#include "HAL/LowLevelMemTracker.h"
#include "IO/IoContainerId.h"
#include "IO/IoContainerHeader.h"
#include "Internationalization/PackageLocalizationManager.h"
#include "Memory/MemoryView.h"
#include "Misc/CommandLine.h"
#include "Misc/ScopeLock.h"
#include "Misc/ScopeRWLock.h"
#include "Misc/PackageName.h"
#include "ProfilingDebugging/CountersTrace.h"
#include "Templates/Sorting.h"

DEFINE_LOG_CATEGORY_STATIC(LogFilePackageStore, Log, All);

namespace UE::FilePackageStorePrivate
{

static constexpr uint32 QuarterBits = 15;
static constexpr uint32 HalfBits = 32;
static constexpr uint32 MinSlotBits = 64 - HalfBits - QuarterBits;
static constexpr uint32 MaxSlotBits = 32;
static constexpr uint32 MinSlots = 1u << MinSlotBits;
static constexpr uint16 QuarterSlotEndMask = uint16(1) << QuarterBits;
static constexpr uint16 QuarterMask = QuarterSlotEndMask - 1;
static constexpr uint32 EmptySlotValue = 0xFFFFFFFF;

static constexpr bool IsSlotEnd(uint16 Quarter)
{
	return Quarter & QuarterSlotEndMask;
}

struct FPackageIdKey
{
	uint32 SlotIndex;
	uint32 Half;
	uint16 Quarter;
};

static uint32 GetSlotSortRadix(FPackageId PID)
{
	return static_cast<uint32>(PID.Value() >> 32);
}

static FPackageIdKey Split(FPackageId PID, uint32 SlotBits)
{
	check(PID.IsValid());
	check(SlotBits >= MinSlotBits && SlotBits <= MaxSlotBits);
	FPackageIdKey Out;
	Out.SlotIndex	= static_cast<uint32>(PID.Value() >> (64 - SlotBits)); // Can overlap with Half value
	Out.Half		= static_cast<uint32>(PID.Value() >> QuarterBits);
	Out.Quarter		= static_cast<uint16>(PID.Value()) & QuarterMask;
	return Out;
}

static FPackageId Fuse(FPackageIdKey Key, uint32 SlotBits)
{
	uint64 SlotPart		= uint64(Key.SlotIndex) << (64 - SlotBits);
	uint64 HalfPart		= uint64(Key.Half) << QuarterBits;
	uint64 QuarterPart	= uint64(Key.Quarter & QuarterMask);
	return FPackageId::FromValue(QuarterPart | HalfPart | SlotPart);
}

static uint32 GetNumSlots(uint32 NumValues)
{
	uint32 AvgValuesPerSlot = 4; // 400% load factor
	uint32 Out = FMath::RoundUpToPowerOfTwo(NumValues / AvgValuesPerSlot);
	return FMath::Max(MinSlots, Out);
}

static bool operator==(FEntryHandle A, FEntryHandle B)
{
	static_assert(sizeof(FEntryHandle) == sizeof(uint32));
	return reinterpret_cast<uint32&>(A) == reinterpret_cast<uint32&>(B);
}

FORCENOINLINE void SortByEntryOffset(TArray<TPair<FPackageId, FEntryHandle>>& Pairs)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SortByEntryOffset);
	TArray<TPair<FPackageId, FEntryHandle>> Sorted;
	Sorted.SetNumUninitialized(Pairs.Num());
	RadixSort32(Sorted.GetData(), Pairs.GetData(), Pairs.Num(), [](TPair<FPackageId, FEntryHandle> P) { return P.Value.Offset; } );
	Swap(Sorted, Pairs);
}

FORCENOINLINE void SortBySlotIndex(TArray<TPair<FPackageId, FEntryHandle>>& Pairs)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SortBySlotIndex);
	TArray<TPair<FPackageId, FEntryHandle>> Sorted;
	Sorted.SetNumUninitialized(Pairs.Num());
	RadixSort32(Sorted.GetData(), Pairs.GetData(), Pairs.Num(), [](TPair<FPackageId, FEntryHandle> P) { return GetSlotSortRadix(P.Key); } );
	Swap(Sorted, Pairs);
}

////////////////////////////////////////////////////////////////////////////////

static constexpr uint32 ValuesAlignment = sizeof(FPackageIdKey::Half) / sizeof(FPackageIdKey::Quarter);
static constexpr uint32 BytesPerValue = sizeof(FPackageIdKey::Quarter) + sizeof(FPackageIdKey::Half) + sizeof(FEntryHandle);

FPackageIdMap::FPackageIdMap(TArray<TPair<FPackageId, FEntryHandle>>&& Pairs)
: MaxValues(Align(Pairs.Num(), ValuesAlignment))
, SlotBits(Pairs.IsEmpty() ? 0 : FMath::CountTrailingZeros(GetNumSlots(Pairs.Num())))
, Values((uint16*)FMemory::Malloc(MaxValues * BytesPerValue))
{
	check(SlotBits == 0 || SlotBits >= MinSlotBits && SlotBits <= MaxSlotBits);

	// Allocate Slots and initialize to 0xFFFFFFFF / EmptySlotValue
	Slots = (uint32*)FMemory::Malloc(sizeof(uint32) * NumSlots()); 
	FMemory::Memset(Slots, 0xFF, sizeof(uint32) * NumSlots());

	SortBySlotIndex(Pairs);

	// Populate Slots and Values
	uint16* Quarters		= Values;
	uint32* Halves			= reinterpret_cast<uint32*>(Values + MaxValues);
	FEntryHandle* Entries	= reinterpret_cast<FEntryHandle*>(Halves + MaxValues);
	for (uint32 Idx = 0, Num = Pairs.Num(); Idx < Num; ++Idx)
	{
		FPackageIdKey Key = Split(Pairs[Idx].Key, SlotBits);

		if (Slots[Key.SlotIndex] == EmptySlotValue)
		{
			Slots[Key.SlotIndex] = Idx;
		}
		else
		{
			check(Idx > 0 && IsSlotEnd(Values[Idx - 1]));
			check(Split(Pairs[Idx - 1].Key, SlotBits).SlotIndex == Key.SlotIndex);
			Values[Idx - 1] &= QuarterMask;
		}
		
		Quarters[Idx]	= Key.Quarter | QuarterSlotEndMask;
		Halves[Idx]		= Key.Half;
		Entries[Idx]	= Pairs[Idx].Value;
	}
}
FPackageIdMap& FPackageIdMap::operator=(FPackageIdMap&& O)
{
	if (this != &O)
	{
		Empty();
		FMemory::Memcpy(*this, O);
		FMemory::Memzero(O);
	}
	return *this;
}

void FPackageIdMap::Empty()
{
	FMemory::Free(Slots);
	FMemory::Free(Values);
	FMemory::Memzero(*this);
}

const FEntryHandle* FPackageIdMap::Find(FPackageId InKey) const
{
	if (MaxValues == 0)
	{
		return nullptr;
	}

	FPackageIdKey Key = Split(InKey, SlotBits);
	uint32 Idx = Slots[Key.SlotIndex];

	uint32* Halves			= reinterpret_cast<uint32*>(Values + MaxValues);
	FEntryHandle* Entries	= reinterpret_cast<FEntryHandle*>(Halves + MaxValues);
	
	FPlatformMisc::Prefetch(Values + Idx);
	FPlatformMisc::Prefetch(Halves + Idx);
	FPlatformMisc::Prefetch(Entries + Idx);

	if (Idx == EmptySlotValue)
	{
		return nullptr;
	}

	while (true) 
	{
		check(Idx < MaxValues);
		uint16 Quarter = Values[Idx]; // Consider vectorizing
		if (Key.Quarter != (Quarter & QuarterMask))
		{
			if (IsSlotEnd(Quarter))
			{
				return nullptr;
			}
		}
		else if (Key.Half == Halves[Idx])
		{
			return Entries + Idx;
		}

		++Idx;
	}
}

uint64 FPackageIdMap::GetAllocatedSize() const
{	
	return NumSlots() * sizeof(uint32) + MaxValues * BytesPerValue;
}

class FPackageIdMap::FConstIterator
{
public:
	FConstIterator(const FPackageIdMap& Map)
	: Slots(Map.Slots)
	, Values(Map.Values)
	, SlotBits(Map.SlotBits)
	, NumSlots(Map.NumSlots())
	, MaxValues(Map.MaxValues)
	, ValueIdx(GetSlotValue())
	{}

	explicit operator bool() const
	{
		return SlotIdx < NumSlots;
	}

	TPair<FPackageId, FEntryHandle> operator*() const
	{
		check(*this);

		FEntryHandle Value = GetEntries()[ValueIdx];

		FPackageIdKey Key;
		Key.SlotIndex = SlotIdx;
		Key.Half = GetHalves()[ValueIdx];
		Key.Quarter = Values[ValueIdx];

		return { Fuse(Key, SlotBits), Value };
	}

	void operator++()
	{
		check(*this);

		if (IsSlotEnd(Values[ValueIdx]))
		{
			++SlotIdx;
			ValueIdx = GetSlotValue(); 
		}
		else
		{
			++ValueIdx;
			check(ValueIdx < MaxValues);
		}
	}

private:
	const uint32* Slots;
	const uint16* Values;
	uint32 SlotBits;
	uint32 NumSlots;
	uint32 MaxValues;
	uint32 SlotIdx = 0;
	uint32 ValueIdx;
	
	uint32 GetSlotValue()
	{
		while (SlotIdx < NumSlots && Slots[SlotIdx] == EmptySlotValue)
		{
			++SlotIdx;	
		}
		
		return *this ? Slots[SlotIdx] : 0;
	}

	const uint32* GetHalves() const
	{
		return reinterpret_cast<const uint32*>(Values + MaxValues);
	}

	const FEntryHandle* GetEntries() const
	{
		return reinterpret_cast<const FEntryHandle*>(GetHalves() + MaxValues);
	}
};

static TArray<TPair<FPackageId, FEntryHandle>> ToArray(const FPackageIdMap& Map)
{
	TArray<TPair<FPackageId, FEntryHandle>> Out;
	Out.Reserve(Map.GetCapacity());
	for (FPackageIdMap::FConstIterator It(Map); It; ++It)
	{
		Out.Add(*It);
	}
	return Out;
}

////////////////////////////////////////////////////////////////////////////////

static FStringView GetRootPathFromPackageName(const FStringView Path)
{
	FStringView RootPath;
	int32 SecondForwardSlash = INDEX_NONE;
	if (ensure(FStringView(Path.GetData() + 1, Path.Len() - 1).FindChar(TEXT('/'), SecondForwardSlash)))
	{
		RootPath = FStringView(Path.GetData(), SecondForwardSlash + 2);
	}
	return RootPath;
}

static bool IsEmpty(FEntryHandle Handle) 
{
	return Handle.HasPackageIds + Handle.HasShaderMaps + Handle.HasSoftRefs == 0;
}

} // namespace UE::FilePackageStorePrivate

static bool IsEmpty(const FFilePackageStoreEntry& E, const FFilePackageStoreEntrySoftReferences* SoftRefs)
{
	return (E.ImportedPackages.Num() + E.ShaderMapHashes.Num() == 0) && (SoftRefs == nullptr || SoftRefs->Indices.Num() == 0);
}

////////////////////////////////////////////////////////////////////////////////

void FFilePackageStoreBackend::BeginRead()
{
	EntriesLock.ReadLock();
	if (bNeedsContainerUpdate)
	{
		Update();
	}
}

void FFilePackageStoreBackend::EndRead()
{
	EntriesLock.ReadUnlock();
}

EPackageStoreEntryStatus FFilePackageStoreBackend::GetPackageStoreEntry(FPackageId PackageId, FName PackageName,
	FPackageStoreEntry& OutPackageStoreEntry)
{
	if (const FEntryHandle* Entry = PackageEntries.Find(PackageId))
	{
		if (!IsEmpty(*Entry))
		{
			OutPackageStoreEntry.ImportedPackageIds = GetImportedPackages(*Entry);
			OutPackageStoreEntry.ShaderMapHashes = GetShaderHashes(*Entry);
		}
		
#if WITH_EDITOR
		const FFilePackageStoreEntry* FindOptionalSegmentEntry = OptionalSegmentStoreEntriesMap.FindRef(PackageId);
		if (FindOptionalSegmentEntry)
		{
			OutPackageStoreEntry.OptionalSegmentImportedPackageIds = MakeArrayView(FindOptionalSegmentEntry->ImportedPackages.Data(), FindOptionalSegmentEntry->ImportedPackages.Num());
			OutPackageStoreEntry.bHasOptionalSegment = true;
		}
#endif

		return EPackageStoreEntryStatus::Ok;
	}
		
	return EPackageStoreEntryStatus::Missing;
}

bool FFilePackageStoreBackend::GetPackageRedirectInfo(FPackageId PackageId, FName& OutSourcePackageName, FPackageId& OutRedirectedToPackageId)
{
	TTuple<FName, FPackageId>* FindRedirect = RedirectsPackageMap.Find(PackageId);
	if (FindRedirect)
	{
		OutSourcePackageName = FindRedirect->Get<0>();
		OutRedirectedToPackageId = FindRedirect->Get<1>();
		UE_LOGF(LogFilePackageStore, Verbose, "Redirecting from %ls to 0x%ls", *OutSourcePackageName.ToString(), *LexToString(OutRedirectedToPackageId));
		return true;
	}
	
	if (!GIsEditor) // Skip localized asset redirection in Editor
	{
		const FName* FindLocalizedPackageSourceName = LocalizedPackages.Find(PackageId);
		if (FindLocalizedPackageSourceName)
		{
			FName LocalizedPackageName = FPackageLocalizationManager::Get().FindLocalizedPackageName(*FindLocalizedPackageSourceName);
			if (!LocalizedPackageName.IsNone())
			{
				FPackageId LocalizedPackageId = FPackageId::FromName(LocalizedPackageName);
				if (PackageEntries.Find(LocalizedPackageId))
				{
					OutSourcePackageName = *FindLocalizedPackageSourceName;
					OutRedirectedToPackageId = LocalizedPackageId;
					UE_LOGF(LogFilePackageStore, Verbose, "Redirecting from localized package %ls to 0x%ls", *OutSourcePackageName.ToString(), *LexToString(OutRedirectedToPackageId));
					return true;
				}
			}
		}
	}
	return false;
}

TConstArrayView<uint32> FFilePackageStoreBackend::GetSoftReferences(FPackageId PackageId, TConstArrayView<FPackageId>& OutPackageIds)
{
	if (const FEntryHandle* Entry = PackageEntries.Find(PackageId); Entry != nullptr)
	{
		if (IsEmpty(*Entry))
		{
			OutPackageIds = TConstArrayView<FPackageId>();
			return TConstArrayView<uint32>();
		}

		if (Entry->HasSoftRefs)
		{
			FMountedContainer* Container = Algo::FindByPredicate(
				MountedContainers,
				[Entry](const FMountedContainer& C) { return C.EntryDataRange.Contains(Entry->Offset); });

			if (Container != nullptr)
			{
				// to do: figure out a better way to make sure softreference information wasn't loaded for this container, also this should make the outer request fail completely and error with appropriate information
				if (!Container->ContainerHeader->SoftPackageReferences.bLoadedSoftPackageReferences)
				{
					UE_LOGF(LogFilePackageStore, Error, "Attempting to read softreference information for package %ls but container %ls never loaded softreference information", *LexToString(PackageId), *LexToString(Container->ContainerHeader->ContainerId));
				}
				OutPackageIds = Container->ContainerHeader->SoftPackageReferences.PackageIds;
				return GetSoftReferenceIndices(*Entry);
			}
			else
			{
				UE_LOGF(LogFilePackageStore, Warning, "Failed to find container with soft references for package ID '%ls'", *LexToString(PackageId));
			}
		}
		else
		{
			// make sure tha tit's containers mounted soft refs
//#if !UE_BUILD_SHIPPING
//			FMountedContainer* Container = Algo::FindByPredicate(
//				MountedContainers,
//				[Entry](const FMountedContainer& C) { return C.EntryDataRange.Contains(Entry->Offset); });
//
//			if (Container != nullptr)
//			{
//				// to do: figure out a better way to make sure softreference information wasn't loaded for this container, also this should make the outer request fail completely and error with appropriate information
//				if (!Container->ContainerHeader->SoftPackageReferences.bLoadedSoftPackageReferences)
//				{
//					UE_LOGF(LogFilePackageStore, Error, "Attempting to read softreference information for package %ls but container %ls never loaded softreference information", *LexToString(PackageId), *LexToString(Container->ContainerHeader->ContainerId));
//				}
//			}
//#endif
		}
	}

	return TConstArrayView<uint32>();
}

void FFilePackageStoreBackend::Mount(FIoContainerHeader* ContainerHeader, uint32 Order)
{
	LLM_SCOPE_BYNAME(TEXT("Filesystem/FilePackageStore"));
	FWriteScopeLock _(EntriesLock);
	MountedContainers.Add({ ContainerHeader, Order, NextSequence++ });
	// Update() relies on stable sorting to efficiently drop unmounted package -> entry ranges, see OldIt
	Algo::StableSort(MountedContainers, [](const FMountedContainer& A, const FMountedContainer& B)
		{
			if (A.Order == B.Order)
			{
				return A.Sequence > B.Sequence;
			}
			return A.Order > B.Order;
		});
	bNeedsContainerUpdate = true;

	UE_LOGF(LogFilePackageStore, Log, "Mounting container: Id=%ls, Order=%u, NumPackages=%d",
		*LexToString(ContainerHeader->ContainerId), Order, ContainerHeader->PackageIds.Num());
}

void FFilePackageStoreBackend::Unmount(const FIoContainerHeader* ContainerHeader)
{
	FWriteScopeLock _(EntriesLock);
	for (auto It = MountedContainers.CreateIterator(); It; ++It)
	{
		if (It->ContainerHeader == ContainerHeader)
		{
			It.RemoveCurrent();
			bNeedsContainerUpdate = true;
			return;
		}
	}
}

template<class T>
constexpr uint32 GetNumWords(uint32 Num)
{
	static_assert(sizeof(T) % sizeof(uint32) == 0);
	return (sizeof(T) / sizeof(uint32)) * Num;
}

FFilePackageStoreBackend::FEntryHandle FFilePackageStoreBackend::AddNewEntryData(const FFilePackageStoreEntry& Entry, const FFilePackageStoreEntrySoftReferences* SoftReferences)
{
	check(EntryData.Num() < (1u << FEntryHandle::OffsetBits));
	FEntryHandle Out;
	Out.Offset = EntryData.Num();

	if (IsEmpty(Entry, SoftReferences))
	{
		EntryData.Add(0u);
	}
	else
	{
		auto AppendEntryData = [this](const auto* Data, uint32 Num)
		{
			EntryData.Add(Num);
			EntryData.Append(reinterpret_cast<const uint32*>(Data), GetNumWords<decltype(*Data)>(Num));
		};
	
		if (uint32 Num = Entry.ImportedPackages.Num())
		{
			Out.HasPackageIds = 1;
			AppendEntryData(Entry.ImportedPackages.Data(), Num);
		}

		if (uint32 Num = Entry.ShaderMapHashes.Num())
		{
			Out.HasShaderMaps = 1;
			AppendEntryData(Entry.ShaderMapHashes.Data(), Num);
		}

		if (uint32 Num = SoftReferences != nullptr ? SoftReferences->Indices.Num() : 0)
		{
			Out.HasSoftRefs = 1;
			AppendEntryData(SoftReferences->Indices.Data(), Num);
		}
	}

	check((uint32)EntryData.Num() > Out.Offset);
	return Out;
}

FFilePackageStoreBackend::FEntryHandle FFilePackageStoreBackend::AddOldEntryData(FEntryHandle OldHandle, TConstArrayView<uint32> OldEntryData)
{
	check(EntryData.Num() < (1u << FEntryHandle::OffsetBits));
	FEntryHandle Out = OldHandle;
	Out.Offset = EntryData.Num();

	if (IsEmpty(OldHandle))
	{
		EntryData.Add(0u);
	}
	else
	{
		const uint32* It = &OldEntryData[OldHandle.Offset];
		uint32 NumPackageWords = OldHandle.HasPackageIds ? 1 + GetNumWords<FPackageId>(*It)  : 0;
		check(It + NumPackageWords <= OldEntryData.end())
		EntryData.Append(It, NumPackageWords);

		It += NumPackageWords;

		uint32 NumShaderWords = OldHandle.HasShaderMaps ? 1 + GetNumWords<FShaderHash>(*It)  : 0;
		check(It + NumShaderWords <= OldEntryData.end())
		EntryData.Append(It, NumShaderWords);

		It += NumShaderWords;

		uint32 NumSoftRefWords = OldHandle.HasSoftRefs ? 1 + GetNumWords<uint32>(*It)  : 0;
		check(It + NumSoftRefWords <= OldEntryData.end())
		EntryData.Append(It, NumSoftRefWords);
	}
	
	check((uint32)EntryData.Num() > Out.Offset);
	return Out;
}

template<class T>
TConstArrayView<T> MakeEntryDataSlice(const uint32* NumHeader, TConstArrayView<uint32> Bounds)
{
	check(NumHeader >= Bounds.GetData());
	TConstArrayView<T> Out(reinterpret_cast<const T*>(NumHeader + 1), *NumHeader);
	check(Out.Num() > 0);
	check(reinterpret_cast<const uint32*>(Out.end()) <= Bounds.end());
	return Out;
}

TConstArrayView<FPackageId> FFilePackageStoreBackend::GetImportedPackages(FEntryHandle Handle)
{
	if (Handle.HasPackageIds == 0)
	{
		return {};
	}

	return MakeEntryDataSlice<FPackageId>(&EntryData[Handle.Offset], EntryData);
}

TConstArrayView<FShaderHash> FFilePackageStoreBackend::GetShaderHashes(FEntryHandle Handle)
{
	if (Handle.HasShaderMaps == 0)
	{
		return {};
	}

	const uint32* Data = &EntryData[Handle.Offset];
	uint32 NumPackageIdWords = Handle.HasPackageIds ? 1 + GetNumWords<FPackageId>(*Data) : 0;
	return MakeEntryDataSlice<FShaderHash>(Data + NumPackageIdWords, EntryData);
}

TConstArrayView<uint32> FFilePackageStoreBackend::GetSoftReferenceIndices(FEntryHandle Handle)
{
	if (Handle.HasSoftRefs == 0)
	{
		return {};
	}

	const uint32* Data = &EntryData[Handle.Offset];
	uint32 NumPackageIdAndHashWords = Handle.HasPackageIds ? 1 + GetNumWords<FPackageId>(*Data) : 0;
	if (Handle.HasShaderMaps)
	{
		NumPackageIdAndHashWords += (1 + GetNumWords<FShaderHash>(*(Data + NumPackageIdAndHashWords)));
	}
	return MakeEntryDataSlice<uint32>(Data + NumPackageIdAndHashWords, EntryData);
}

// Update() entry deduplication helper
struct FFilePackageStoreEntryRef
{
	const FFilePackageStoreEntry* Entry;
	const FFilePackageStoreEntrySoftReferences* SoftReferences = nullptr;

	FMemoryView GetImportedPackages() const
	{
		return MakeMemoryView(Entry->ImportedPackages.begin(), Entry->ImportedPackages.end());
	}

	FMemoryView GetShaderHashes() const
	{
		return MakeMemoryView(Entry->ShaderMapHashes.begin(), Entry->ShaderMapHashes.end());
	}

	FMemoryView GetSoftReferences() const
	{
		return SoftReferences != nullptr ? FMemoryView(SoftReferences->Indices.begin(), SoftReferences->Indices.end()) : FMemoryView();
	}

	bool operator==(FFilePackageStoreEntryRef O) const
	{
		return GetImportedPackages().EqualBytes(O.GetImportedPackages()) && GetShaderHashes().EqualBytes(O.GetShaderHashes()) && GetSoftReferences().EqualBytes(O.GetSoftReferences());
	}

	friend uint32 GetTypeHash(FFilePackageStoreEntryRef Ref)
	{
		uint32 Out = 0;

		for (FPackageId PID : Ref.Entry->ImportedPackages)
		{
			Out = HashCombineFast(Out, static_cast<uint32>(PID.Value()));
		}

		for (const FShaderHash& ShaderMapHash : Ref.Entry->ShaderMapHashes)
		{
			Out = HashCombineFast(Out, static_cast<uint32>(ShaderMapHash.Hash));
		}

		if (Ref.SoftReferences != nullptr)
		{
			for (uint32 Idx : Ref.SoftReferences->Indices)
			{
				Out = HashCombineFast(Out, Idx); 
			}
		}

		return Out;
	}
};

void FFilePackageStoreBackend::Update()
{
	LLM_SCOPE_BYNAME(TEXT("Filesystem/FilePackageStore"));
	TRACE_CPUPROFILER_EVENT_SCOPE(UpdateFilePackageStore);

	FScopeLock Lock(&UpdateLock);
	if (bNeedsContainerUpdate)
	{	
		// Count packages
		uint32 TotalNewPackages = 0;
		uint32 TotalOldPackages = 0;
		for (const FMountedContainer& MountedContainer : MountedContainers)
		{
			TotalNewPackages += MountedContainer.ContainerHeader->PackageIds.Num();
			TotalOldPackages += MountedContainer.NumMountedPackages;
		}

		// Set up temporary data structures
		TArray<TPair<FPackageId, FEntryHandle>> OldPairs = ToArray(PackageEntries);
		TMap<FFilePackageStoreEntryRef, FEntryHandle> UniqueEntriesPerNewContainer;
		TMap<uint32, FEntryHandle> UniqueEntriesPerOldContainer;
		TArray<TPair<FPackageId, FEntryHandle>> Pairs;
		SortByEntryOffset(OldPairs);
		UniqueEntriesPerNewContainer.Reserve(TotalNewPackages / 2);
		UniqueEntriesPerOldContainer.Reserve(TotalOldPackages / 2);
		Pairs.Reserve(TotalNewPackages + TotalOldPackages);

		// Empty old containers but keep old pairs and entry data around temporarily
		LocalizedPackages.Empty();
		RedirectsPackageMap.Empty();
#if WITH_EDITOR
		OptionalSegmentStoreEntriesMap.Empty();
#endif
		TArray<uint32> OldEntryData;
		Swap(EntryData, OldEntryData);
		EntryData.Reserve(static_cast<int32>(OldEntryData.Num() + 32 * TotalNewPackages));
	
		// Helper constants
		const uint32 IllegalHandleValue = ~0u;
		const FEntryHandle IllegalHandle = reinterpret_cast<const FEntryHandle&>(IllegalHandleValue);
		const FFilePackageStoreEntry EmptyEntry;
		
		const TPair<FPackageId, FEntryHandle>* OldIt = OldPairs.GetData();
		const TPair<FPackageId, FEntryHandle>* OldEnd = OldIt + OldPairs.Num();
		for (FMountedContainer& MountedContainer : MountedContainers)
		{
			FIoContainerHeader* ContainerHeader = MountedContainer.ContainerHeader;
			
			const FMountedDataRange OldMountedEntries = MountedContainer.EntryDataRange;
			MountedContainer.EntryDataRange.Begin = EntryData.Num();

			if (MountedContainer.NumMountedPackages > 0)
			{
				UniqueEntriesPerOldContainer.Reset();

				// Skip unmounted ranges
				for (; OldIt != OldEnd && OldIt->Value.Offset < OldMountedEntries.Begin; ++OldIt);

				// Re-add old package entries with new entry handle
				for (; OldIt != OldEnd && OldIt->Value.Offset < OldMountedEntries.End; ++OldIt)
				{
					check(OldIt->Value.Offset >= OldMountedEntries.Begin);

					FEntryHandle& NewHandle = UniqueEntriesPerOldContainer.FindOrAdd(OldIt->Value.Offset, IllegalHandle);
					if (NewHandle == IllegalHandle)
					{
						NewHandle = AddOldEntryData(OldIt->Value, OldEntryData);
					}

					Pairs.Add({OldIt->Key, NewHandle});
				}
			}
			else if (int32 NumNewPackages = ContainerHeader->PackageIds.Num())
			{
				check(UniqueEntriesPerNewContainer.IsEmpty());

				// Add an empty entry per mounted container to allow inverse mapping handle offset -> container,
				// which in turn allows releasing FIoContainerHeader::PackageIds after first Update()
				UniqueEntriesPerNewContainer.Add({&EmptyEntry}, AddNewEntryData(EmptyEntry, nullptr));

				TConstArrayView<FFilePackageStoreEntrySoftReferences> AllSoftReferences;
				if (ContainerHeader->SoftPackageReferences.bContainsSoftPackageReferences)
				{
					AllSoftReferences = MakeArrayView<const FFilePackageStoreEntrySoftReferences>(
						reinterpret_cast<const FFilePackageStoreEntrySoftReferences*>(ContainerHeader->SoftPackageReferences.PackageIndices.GetData()),
						ContainerHeader->PackageIds.Num());
				}

				TConstArrayView<FFilePackageStoreEntry> PackageStoreEntries = MakeArrayView(reinterpret_cast<const FFilePackageStoreEntry*>(ContainerHeader->StoreEntries.GetData()), NumNewPackages);
				for (int32 Idx = 0; const FFilePackageStoreEntry& StoreEntry : PackageStoreEntries)
				{
					const FFilePackageStoreEntrySoftReferences* SoftReferences = AllSoftReferences.IsEmpty() ? nullptr : &AllSoftReferences[Idx];
					FEntryHandle& Handle = UniqueEntriesPerNewContainer.FindOrAdd({&StoreEntry, SoftReferences}, IllegalHandle);
					if (Handle == IllegalHandle)
					{
						Handle = AddNewEntryData(StoreEntry, SoftReferences);
					}

					const FPackageId& PackageId = ContainerHeader->PackageIds[Idx++];
					Pairs.Add({PackageId, Handle});
				}

				// Clear container for next use before clearing the referenced data to avoid use-after-free
				UniqueEntriesPerNewContainer.Reset();

				MountedContainer.NumMountedPackages = NumNewPackages;
				ContainerHeader->PackageIds.Empty();
				ContainerHeader->SoftPackageReferences.PackageIndices.Empty();
			}

			
			MountedContainer.EntryDataRange.End = EntryData.Num();
			check(MountedContainer.EntryDataRange.End > MountedContainer.EntryDataRange.Begin  || MountedContainer.NumMountedPackages == 0);

#if WITH_EDITOR
			TArrayView<const FFilePackageStoreEntry> ContainerOptionalSegmentStoreEntries(reinterpret_cast<const FFilePackageStoreEntry*>(ContainerHeader->OptionalSegmentStoreEntries.GetData()), ContainerHeader->OptionalSegmentPackageIds.Num());
			int32 Index = 0;
			for (const FFilePackageStoreEntry& OptionalSegmentStoreEntry : ContainerOptionalSegmentStoreEntries)
			{
				const FPackageId& PackageId = ContainerHeader->OptionalSegmentPackageIds[Index];
				check(PackageId.IsValid());
				OptionalSegmentStoreEntriesMap.FindOrAdd(PackageId, &OptionalSegmentStoreEntry);
				++Index;
			}
#endif //if WITH_EDITOR

			for (const FIoContainerHeaderLocalizedPackage& LocalizedPackage : ContainerHeader->LocalizedPackages)
			{
				FName& LocalizedPackageSourceName = LocalizedPackages.FindOrAdd(LocalizedPackage.SourcePackageId);
				if (LocalizedPackageSourceName.IsNone())
				{
					FDisplayNameEntryId NameEntry = ContainerHeader->RedirectsNameMap[LocalizedPackage.SourcePackageName.GetIndex()];
					LocalizedPackageSourceName = NameEntry.ToName(LocalizedPackage.SourcePackageName.GetNumber());
				}
			}

			for (const FIoContainerHeaderPackageRedirect& Redirect : ContainerHeader->PackageRedirects)
			{
				TTuple<FName, FPackageId>& RedirectEntry = RedirectsPackageMap.FindOrAdd(Redirect.SourcePackageId);
				FName& SourcePackageName = RedirectEntry.Key;
				if (SourcePackageName.IsNone())
				{
					FDisplayNameEntryId NameEntry = ContainerHeader->RedirectsNameMap[Redirect.SourcePackageName.GetIndex()];
					SourcePackageName = NameEntry.ToName(Redirect.SourcePackageName.GetNumber());
					RedirectEntry.Value = Redirect.TargetPackageId;
				}
			}
		}
		
		// Clear old containers and release all FIoContainerHeader::StoreEntries 
		UniqueEntriesPerNewContainer.Empty();
		PackageEntries.Empty();
		OldEntryData.Empty();
		for (FMountedContainer& MountedContainer : MountedContainers)
		{
			MountedContainer.ContainerHeader->StoreEntries.Empty();
			check(MountedContainer.ContainerHeader->PackageIds.GetAllocatedSize() == 0);
		}

		// Finally reallocate entry data and build new map
		EntryData.Shrink();
		LocalizedPackages.Shrink();
		RedirectsPackageMap.Shrink();
#if WITH_EDITOR
		OptionalSegmentStoreEntriesMap.Shrink();
#endif
		PackageEntries = FPackageIdMap(MoveTemp(Pairs));

		bNeedsContainerUpdate = false;	

		UE_LOGF(LogFilePackageStore, Log, "Updated: NewPackages=%u, OldPackages=%u, TotalPackages=%u",
			TotalNewPackages, TotalOldPackages, PackageEntries.GetCapacity());
	}
}

//////////////////////////////////////////////////////////////////////////

#if WITH_TESTS

#include "Tests/TestHarnessAdapter.h"

using namespace UE::FilePackageStorePrivate;

TEST_CASE_NAMED(FPackageIdMapTest, "System::Pak::IoStore::PackageIdMap", "[Pak][IoStore][SmokeFilter]")
{
	SECTION("PackageIdKey")
	{
		static constexpr uint64 Values[] = {
			0b00000000'00000000'00000000'00000000'00000000'00000000'00000000'00000001,
			0b10000000'00000000'00000000'00000000'00000000'00000000'00000000'00000000,
			0b10101010'10101010'10101010'10101010'10101010'10101010'10101010'10101010,
			0b11001100'11001100'11001100'11001100'11001100'11001100'11001100'11001100,
			0b11110000'11110000'11110000'11110000'11110000'11110000'11110000'11110000,
			0b11111111'00000000'11111111'00000000'11111111'00000000'11111111'00000000,
			0b11111111'11111111'11111111'11111111'00000000'00000000'00000000'00000000,
		};

		for (uint32 SlotBits : {17, 18, 19, 24, 31, 32})
		{
			for (uint64 Value : Values)
			{
				CHECK(FPackageId::FromValue( Value) == Fuse(Split(FPackageId::FromValue( Value), SlotBits), SlotBits));
				CHECK(FPackageId::FromValue(~Value) == Fuse(Split(FPackageId::FromValue(~Value), SlotBits), SlotBits));
			}
		}
	}

	auto MakeTestHandle = [](uint32 Offset)
	{
		FEntryHandle Out;
		Out.Offset = Offset;
		Out.HasPackageIds = Offset & 1;
		Out.HasShaderMaps = Offset/2  & 1;
		return Out;
	};

	SECTION("Sorting")
	{
		const std::initializer_list<TPair<FPackageId, FEntryHandle>> Unsorted = {
			{FPackageId::FromValue(0x00000000'00000002), MakeTestHandle(2)},
			{FPackageId::FromValue(0x00000000'00000001), MakeTestHandle(1)},
			{FPackageId::FromValue(0xfedcba09'87654321), MakeTestHandle(0x123456)},
			{FPackageId::FromValue(0x00000000'00000003), MakeTestHandle(1u << 28)},
			{FPackageId::FromValue(0x00000000'10000000), MakeTestHandle(1)},
			{FPackageId::FromValue(0x00000001'00000000), MakeTestHandle(2)},
			{FPackageId::FromValue(0x00000010'00000000), MakeTestHandle(3)},
			{FPackageId::FromValue(0x00000100'00000000), MakeTestHandle(4)},
			{FPackageId::FromValue(0x00001000'00000000), MakeTestHandle(1)},
			{FPackageId::FromValue(0x00010000'00000000), MakeTestHandle(2)},
			{FPackageId::FromValue(0x00100000'00000000), MakeTestHandle(3)},
			{FPackageId::FromValue(0x01000000'00000000), MakeTestHandle(4)},
			{FPackageId::FromValue(0x10000000'00000000), MakeTestHandle(5)},
			{FPackageId::FromValue(0x01100000'00000000), MakeTestHandle(6)},
		};
		
		TArray<TPair<FPackageId, FEntryHandle>> OffsetSorted = Unsorted;
		SortByEntryOffset(OffsetSorted);
		for (uint32 Idx = 0; Idx + 1 < Unsorted.size(); ++Idx)
		{
			CHECK(OffsetSorted[Idx].Value.Offset <= OffsetSorted[Idx + 1].Value.Offset);			
		}

		TArray<TPair<FPackageId, FEntryHandle>> SlotSorted = Unsorted;
		SortBySlotIndex(SlotSorted);
		for (uint32 Idx = 0; Idx + 1 < Unsorted.size(); ++Idx)
		{
			CHECK(Split(SlotSorted[Idx].Key, MinSlotBits).SlotIndex <= Split(SlotSorted[Idx + 1].Key, MinSlotBits).SlotIndex);			
			CHECK(Split(SlotSorted[Idx].Key, MaxSlotBits).SlotIndex <= Split(SlotSorted[Idx + 1].Key, MaxSlotBits).SlotIndex);			
		}
	}
	
	auto MakeMap = [](TConstArrayView<TPair<FPackageId, FEntryHandle>> Pairs)
	{
		TMap<FPackageId, FEntryHandle> Out;
		Out.Reserve(Pairs.Num());
		for (TPair<FPackageId, FEntryHandle> Pair : Pairs)
		{
			Out.Add(Pair);
		}
		return Out;
	};

	SECTION("Map")
	{
		TArray<FPackageId> Pids;
		Pids.Add(FPackageId::FromValue(0x01234567'89abcdef));
		Pids.Add(FPackageId::FromValue(0xfedcba98'76543210));
		for (uint32 Idx = 0; Idx < 63; ++ Idx)
		{
			Pids.Add(FPackageId::FromValue(uint64(1) << Idx));
		}
		for (int32 Idx = 0, Num = Pids.Num(); Idx < Num; ++Idx)
		{
			Pids.Add(FPackageId::FromValue(Pids[Idx].Value() + 1));
		}
		for (int32 Idx = 0, Num = Pids.Num(); Idx < Num; ++Idx)
		{
			Pids.Add(FPackageId::FromValue(~Pids[Idx].Value()));
		}
		
		TArray<TPair<FPackageId, FEntryHandle>> Pairs;
		Pairs.Reserve(Pids.Num());
		for (FPackageId Pid : Pids)
		{
			Pairs.Add({Pid, reinterpret_cast<const FEntryHandle&>(Pid)});
		}

		TMap<FPackageId, FEntryHandle> Map = MakeMap(Pairs);	
		FPackageIdMap PidMap(MoveTemp(Pairs));
		CHECK(Map.OrderIndependentCompareEqual(MakeMap(ToArray(PidMap))));
		for (TPair<FPackageId, FEntryHandle> Pair : Map)
		{
			CHECK(PidMap.Find(Pair.Key));
			CHECK(*PidMap.Find(Pair.Key) == Pair.Value);
		}
	}
}

#endif
