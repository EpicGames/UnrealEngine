// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IO/PackageId.h"
#include "Serialization/PackageStore.h"
#include "Misc/PackagePath.h"

struct FIoContainerHeader;
struct FFilePackageStoreEntry;
struct FFilePackageStoreEntrySoftReferences;

namespace UE::FilePackageStorePrivate
{
	
struct FMountedDataRange
{
	bool Contains(uint32 Offset) const { return Offset >= Begin && Offset <= End; }

	uint32 Begin = 0;
	uint32 End = 0;
};

struct FMountedContainer
{
	FIoContainerHeader* ContainerHeader;
	uint32 Order;
	uint32 Sequence;
	uint32 NumMountedPackages = 0;
	FMountedDataRange EntryDataRange;
};

struct FEntryHandle
{
	static constexpr uint32 OffsetBits = 29;

	uint32 Offset : OffsetBits;
	uint32 HasPackageIds : 1;
	uint32 HasShaderMaps : 1;
	uint32 HasSoftRefs : 1;

	FEntryHandle()
	{
		FMemory::Memzero(*this);
	}
};

// Memory-optimized immutable FPackageId -> FEntryHandle hash map
class FPackageIdMap
{
public:
	FPackageIdMap() = default;
	explicit FPackageIdMap(TArray<TPair<FPackageId, FEntryHandle>>&& Pairs);
	FPackageIdMap(FPackageIdMap&& O) { *this = MoveTemp(O); }
	FPackageIdMap& operator=(FPackageIdMap&& O);
	~FPackageIdMap() { Empty(); }
	
	void Empty();
	const FEntryHandle* Find(FPackageId Key) const;
	uint32 GetCapacity() const { return MaxValues; }
	uint64 GetAllocatedSize() const;
	class FConstIterator;

private:
	uint32 MaxValues = 0;
	uint32 SlotBits = 0;
	uint32* Slots = nullptr;
	uint16* Values = nullptr;
	
	uint32 NumSlots() const
	{
		return SlotBits ? (1u << SlotBits) : 0u;
	}
};

struct FUncookedPackage
{
	FName PackageName;
	EPackageExtension HeaderExtension;
};

} // namespace UE::FilePackageStorePrivate

/*
 * File/container based package store.
 */
class FFilePackageStoreBackend final : public IPackageStoreBackend
{
public:
	virtual void OnMounted(TSharedRef<const FPackageStoreBackendContext>) override {}
	virtual void BeginRead() override;
	virtual void EndRead() override;
	virtual EPackageStoreEntryStatus GetPackageStoreEntry(FPackageId PackageId, FName PackageName, FPackageStoreEntry& OutPackageStoreEntry) override;
	virtual bool GetPackageRedirectInfo(FPackageId PackageId, FName& OutSourcePackageName, FPackageId& OutRedirectedToPackageId) override;
	virtual TConstArrayView<uint32> GetSoftReferences(FPackageId PackageId, TConstArrayView<FPackageId>& OutPackageIds) override;

	void Mount(FIoContainerHeader* ContainerHeader, uint32 Order);
	void Unmount(const FIoContainerHeader* ContainerHeader);

private:
	using FMountedDataRange = UE::FilePackageStorePrivate::FMountedDataRange;
	using FMountedContainer = UE::FilePackageStorePrivate::FMountedContainer;
	using FEntryHandle = UE::FilePackageStorePrivate::FEntryHandle;
	using FPackageIdMap = UE::FilePackageStorePrivate::FPackageIdMap;
	using FUncookedPackage = UE::FilePackageStorePrivate::FUncookedPackage;

	FRWLock EntriesLock;
	FCriticalSection UpdateLock;
	TArray<FMountedContainer> MountedContainers;
	TAtomic<uint32> NextSequence{ 0 };

	FPackageIdMap PackageEntries;
	TArray<uint32> EntryData;
	TMap<FPackageId, TTuple<FName, FPackageId>> RedirectsPackageMap;
	TMap<FPackageId, FName> LocalizedPackages;
	bool bNeedsContainerUpdate = false;

#if WITH_EDITOR
	FDelegateHandle OnContentPathMountedDelegateHandle;
	FDelegateHandle OnContentPathDismountedDelegateHandle;
	FCriticalSection UncookedPackageRootsLock;
	TSet<FString> PendingAddUncookedPackageRoots;
	TSet<FString> PendingRemoveUncookedPackageRoots;
	TMap<FPackageId, FUncookedPackage> UncookedPackagesMap;
	TMap<FPackageId, const FFilePackageStoreEntry*> OptionalSegmentStoreEntriesMap;
	bool bNeedsUncookedPackagesUpdate = false;

	uint64 AddUncookedPackagesFromRoot(const FString& RootPath);
	uint64 RemoveUncookedPackagesFromRoot(const TSet<FString>& RootPath);
#endif //if WITH_EDITOR

	void Update();
	FEntryHandle AddNewEntryData(const FFilePackageStoreEntry& Entry, const FFilePackageStoreEntrySoftReferences* SoftReferences);
	FEntryHandle AddOldEntryData(FEntryHandle OldHandle, TConstArrayView<uint32> OldEntryData);
	TConstArrayView<FPackageId> GetImportedPackages(FEntryHandle Handle);
	TConstArrayView<FShaderHash> GetShaderHashes(FEntryHandle Handle);
	TConstArrayView<uint32> GetSoftReferenceIndices(FEntryHandle Handle);
};
