// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenericPlatformIoDispatcher.h"
#include "IoDispatcherFileBackend.h"
#include "Async/AsyncFileHandle.h"
#include "HAL/Event.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/RunnableThread.h"
#include "HAL/IConsoleManager.h"
#include "Misc/ScopeLock.h"

//PRAGMA_DISABLE_OPTIMIZATION

bool GIoDispatcherCullCancelledReadRequests = true;
static FAutoConsoleVariableRef CVar_IoDispatcherCullCancelledReadRequests(
	TEXT("s.IoDispatcherCullCancelledReadRequests"),
	GIoDispatcherCullCancelledReadRequests,
	TEXT("Process cancelled file read requests before starting I/O.")
	TEXT("This can prevent the I/O dispatcher thread to stall due to the request queue getting too big.")
);

TRACE_DECLARE_INT_COUNTER_EXTERN(IoDispatcherFileBackendSequentialReads);
TRACE_DECLARE_INT_COUNTER_EXTERN(IoDispatcherFileBackendForwardSeeks);
TRACE_DECLARE_INT_COUNTER_EXTERN(IoDispatcherFileBackendBackwardSeeks);
TRACE_DECLARE_INT_COUNTER_EXTERN(IoDispatcherFileBackendSwitchContainerSeeks);
TRACE_DECLARE_MEMORY_COUNTER_EXTERN(IoDispatcherFileBackendTotalSeekDistance);
TRACE_DECLARE_INT_COUNTER_EXTERN(IoDispatcherFileBackendFileSystemRequests);
TRACE_DECLARE_MEMORY_COUNTER_EXTERN(IoDispatcherFileBackendFileSystemTotalBytesRead);

FGenericFileIoStoreEventQueue::FGenericFileIoStoreEventQueue()
	: ServiceEvent(FPlatformProcess::GetSynchEventFromPool())
{
}

FGenericFileIoStoreEventQueue::~FGenericFileIoStoreEventQueue()
{
	FPlatformProcess::ReturnSynchEventToPool(ServiceEvent);
}

void FGenericFileIoStoreEventQueue::ServiceNotify()
{
	ServiceEvent->Trigger();
}

void FGenericFileIoStoreEventQueue::ServiceWait()
{
	ServiceEvent->Wait();
}

FGenericFileIoStoreImpl::FGenericFileIoStoreImpl()
{
}

FGenericFileIoStoreImpl::~FGenericFileIoStoreImpl()
{
}

bool FGenericFileIoStoreImpl::OpenContainer(const TCHAR* ContainerFilePath, uint64& ContainerFileHandle, uint64& ContainerFileSize)
{
	IPlatformFile* Ipf = nullptr;
	if (UE::IsUsingZenPakFileStreaming())
	{
		Ipf = &FPlatformFileManager::Get().GetPlatformFile();
	}
	else
	{
		Ipf = &IPlatformFile::GetPlatformPhysical();
	}

	const int64 FileSize = Ipf->FileSize(ContainerFilePath);
	if (FileSize < 0)
	{
		return false;
	}
	IFileHandle* FileHandle = Ipf->OpenReadNoBuffering(ContainerFilePath);
	if (!FileHandle)
	{
		return false;
	}
	ContainerFileHandle = reinterpret_cast<UPTRINT>(FileHandle);
	ContainerFileSize = uint64(FileSize);
	return true;
}

void FGenericFileIoStoreImpl::CloseContainer(uint64 ContainerFileHandle)
{
	check(ContainerFileHandle);
	IFileHandle* FileHandle = reinterpret_cast<IFileHandle*>(ContainerFileHandle);
	delete FileHandle;
}

bool FGenericFileIoStoreImpl::StartRequests(FFileIoStoreRequestQueue& RequestQueue)
{
	if (GIoDispatcherCullCancelledReadRequests)
	{
		TArray<FFileIoStoreReadRequest*> Cancelled;
		RequestQueue.PopCancelled(Cancelled);
		if (!Cancelled.IsEmpty())
		{
			{
				FScopeLock _(&CompletedRequestsCritical);
				for (FFileIoStoreReadRequest* Request : Cancelled)
				{
					CompletedRequests.Add(Request);
				}
			}
			WakeUpDispatcherThreadDelegate->Execute();
		}
	}

	if (!AcquiredBuffer)
	{
		AcquiredBuffer = BufferAllocator->AllocBuffer();
		if (!AcquiredBuffer)
		{
			return false;
		}
	}

	FFileIoStoreReadRequest* NextRequest = RequestQueue.Pop();
	if (!NextRequest)
	{
		return false;
	}

	if (NextRequest->bCancelled | NextRequest->bFailed)
	{
		{
			FScopeLock _(&CompletedRequestsCritical);
			CompletedRequests.Add(NextRequest);
		}
		WakeUpDispatcherThreadDelegate->Execute();
		return true;
	}

	uint8* Dest;
	check(!NextRequest->ImmediateScatter.Request);
	
	NextRequest->Buffer = AcquiredBuffer;
	AcquiredBuffer = nullptr;
	Dest = NextRequest->Buffer->Memory;

	if (!BlockCache->Read(NextRequest))
	{
		IFileHandle* FileHandle = reinterpret_cast<IFileHandle*>(static_cast<UPTRINT>(NextRequest->ContainerFilePartition->FileHandle));
		 
		Stats->OnFilesystemReadStarted(NextRequest);
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(ReadBlockFromFile);
			NextRequest->bFailed = true;
			int32 RetryCount = 0;
			while (RetryCount++ < 10)
			{
				if (!FileHandle->Seek(NextRequest->Offset))
				{
					UE_LOGF(LogIoDispatcher, Warning, "Failed seeking to offset %lld (Retries: %d)", NextRequest->Offset, (RetryCount - 1));
					continue;
				}
				if (!FileHandle->Read(Dest, NextRequest->Size))
				{
					UE_LOGF(LogIoDispatcher, Warning, "Failed reading %lld bytes at offset %lld (Retries: %d)", NextRequest->Size, NextRequest->Offset, (RetryCount - 1));
					continue;
				}
				NextRequest->bFailed = false;
				Stats->OnFilesystemReadCompleted(NextRequest);
				BlockCache->Store(NextRequest);
				break;
			}
		}
	}
	{
		FScopeLock _(&CompletedRequestsCritical);
		CompletedRequests.Add(NextRequest);
	}
	WakeUpDispatcherThreadDelegate->Execute();
	return true;
}

void FGenericFileIoStoreImpl::GetCompletedRequests(FFileIoStoreReadRequestList& OutRequests)
{
	FScopeLock _(&CompletedRequestsCritical);
	OutRequests.AppendSteal(CompletedRequests);
}
