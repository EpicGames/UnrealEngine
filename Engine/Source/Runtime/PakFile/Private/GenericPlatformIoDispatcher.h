// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/List.h"
#include "HAL/CriticalSection.h"
#include "Templates/Tuple.h"
#include "HAL/Runnable.h"
#include "Containers/Map.h"
#include "IO/IoDispatcher.h"
#include "IoDispatcherFileBackendTypes.h"
#include "ProfilingDebugging/CountersTrace.h"

class FGenericFileIoStoreEventQueue
{
public:
	FGenericFileIoStoreEventQueue();
	~FGenericFileIoStoreEventQueue();
	void ServiceNotify();
	void ServiceWait();

private:
	FEvent* ServiceEvent = nullptr;
};

class FGenericFileIoStoreImpl : public IPlatformFileIoStore
{
public:
	FGenericFileIoStoreImpl();
	~FGenericFileIoStoreImpl();
	void Initialize(const FInitializePlatformFileIoStoreParams& Params) override
	{
		WakeUpDispatcherThreadDelegate = Params.WakeUpDispatcherThreadDelegate;
		BufferAllocator = Params.BufferAllocator;
		BlockCache = Params.BlockCache;
		Stats = Params.Stats;
	}
	bool OpenContainer(const TCHAR* ContainerFilePath, uint64& ContainerFileHandle, uint64& ContainerFileSize) override;
	void CloseContainer(uint64 ContainerFileHandle) override;
	bool CreateCustomRequests(FFileIoStoreResolvedRequest& ResolvedRequest, FFileIoStoreReadRequestList& OutRequests) override
	{
		return false;
	}
	bool StartRequests(FFileIoStoreRequestQueue& RequestQueue) override;
	void GetCompletedRequests(FFileIoStoreReadRequestList& OutRequests) override;

	virtual void ServiceNotify() override
	{
		EventQueue.ServiceNotify();
	}
	virtual void ServiceWait() override
	{
		EventQueue.ServiceWait();
	}
private:
	const FWakeUpIoDispatcherThreadDelegate* WakeUpDispatcherThreadDelegate = nullptr;
	FGenericFileIoStoreEventQueue EventQueue;
	FFileIoStoreBufferAllocator* BufferAllocator = nullptr;
	FFileIoStoreBlockCache* BlockCache = nullptr;
	FFileIoStoreStats* Stats = nullptr;
	FFileIoStoreBuffer* AcquiredBuffer = nullptr;

	FCriticalSection CompletedRequestsCritical;
	FFileIoStoreReadRequestList CompletedRequests;
#if COUNTERSTRACE_ENABLED
	uint64 PreviousFileHandle = 0;
#endif
};

