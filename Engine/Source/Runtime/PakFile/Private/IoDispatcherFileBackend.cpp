// Copyright Epic Games, Inc. All Rights Reserved.

#include "IoDispatcherFileBackend.h"
#include "Misc/CommandLine.h"
#include "Misc/Parse.h"
#include "Misc/ScopeRWLock.h"
#include "Misc/StringBuilder.h"
#include "ProfilingDebugging/AssetMetadataTrace.h"
#include "ProfilingDebugging/CountersTrace.h"
#include "ProfilingDebugging/IoStoreTrace.h"
#include "HAL/PlatformFileManager.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "GenericPlatformIoDispatcher.h"
#include "HAL/IConsoleManager.h"
#include "Async/AsyncWork.h"
#include "Async/MappedFileHandle.h"
#include "Async/UniqueLock.h"
#include "HAL/RunnableThread.h"
#include "Misc/ScopeLock.h"
#include "Misc/Paths.h"
#include "Algo/AllOf.h"
#include "Algo/IsSorted.h"
#include "Algo/MinElement.h"
#include "Templates/Greater.h"
#include "Containers/Ticker.h"
#include "Modules/ModuleManager.h"
#include "IO/IoContainerHeader.h"
#include "IO/IoDispatcherConfig.h"
#include "IO/IoDispatcherFilesystemStats.h"
#include "Serialization/MemoryReader.h"
#include "FileCache/FileCache.h"

//PRAGMA_DISABLE_OPTIMIZATION

float GFileIoStoreUnmountTimeoutSeconds = 5.0;
static FAutoConsoleVariableRef CVar_FileIoStoreUnmountTimeoutSeconds(
	TEXT("iostore.UnmountTimeoutSeconds"),
	GFileIoStoreUnmountTimeoutSeconds,
	TEXT("Number of seconds (1-10) to wait for pending reads before unmounting containers."),
	ECVF_Default
);

bool GFileIoStoreAsyncUnmountEnabled = true;
static FAutoConsoleVariableRef CVar_FileIoStoreAsyncUnmountEnabled(
	TEXT("iostore.AsyncUnmountEnabled"),
	GFileIoStoreAsyncUnmountEnabled,
	TEXT("When enabled containers will be unloaded from memory asynchronously from unmount.")
);

uint32 FFileIoStoreReadRequest::NextSequence = 0;
#if CHECK_IO_STORE_READ_REQUEST_LIST_MEMBERSHIP
uint32 FFileIoStoreReadRequestList::NextListCookie = 0;
#endif
TAtomic<uint32> FFileIoStoreReader::GlobalPartitionIndex{ 0 };
TAtomic<uint32> FFileIoStoreReader::GlobalContainerInstanceId{ 0 };

class FMappedFileProxy final : public IMappedFileHandle
{
public:
	FMappedFileProxy(IMappedFileHandle* InSharedMappedFileHandle, uint64 InSize, uint64 InPartitionSize)
		: IMappedFileHandle(InSize)
		, PartitionSize(InPartitionSize)
		, SharedMappedFileHandle(InSharedMappedFileHandle)
	{
	}

	virtual ~FMappedFileProxy() { }

	virtual IMappedFileRegion* MapRegion(int64 Offset = 0, int64 BytesToMap = MAX_int64, FFileMappingFlags Flags = EMappedFileFlags::ENone) override
	{
		// The offset we get is the TocOffset for the container, and we need the offset into our partition.
		if (PartitionSize)
		{
			Offset %= PartitionSize;
		}
		return SharedMappedFileHandle != nullptr ? SharedMappedFileHandle->MapRegion(Offset, BytesToMap, Flags) : nullptr;
	}
private:
	uint64 PartitionSize;
	IMappedFileHandle* SharedMappedFileHandle;
};

void FFileIoStoreBufferAllocator::Initialize(uint64 InMemorySize, uint64 InBufferSize, uint32 InBufferAlignment)
{
	uint64 BufferCount = InMemorySize / InBufferSize;
	uint64 MemorySize = BufferCount * InBufferSize;
	BufferMemory = reinterpret_cast<uint8*>(FMemory::Malloc(MemorySize, InBufferAlignment));
	BufferSize = InBufferSize;
	for (uint64 BufferIndex = 0; BufferIndex < BufferCount; ++BufferIndex)
	{
		FFileIoStoreBuffer* Buffer = new FFileIoStoreBuffer();
		Buffer->Memory = BufferMemory + BufferIndex * BufferSize;
		Buffer->Next = FirstFreeBuffer;
		FirstFreeBuffer = Buffer;
		Stats.OnBufferReleased();
	}
}

FFileIoStoreBuffer* FFileIoStoreBufferAllocator::AllocBuffer()
{
	FFileIoStoreBuffer* Buffer;
	{
		FScopeLock Lock(&BuffersCritical);
		Buffer = FirstFreeBuffer;
		if (!Buffer)
		{
			return nullptr;
		}
		FirstFreeBuffer = Buffer->Next;
	}
	Stats.OnBufferAllocated();
	return Buffer;
}

void FFileIoStoreBufferAllocator::FreeBuffer(FFileIoStoreBuffer* Buffer)
{
	check(Buffer);
	{
		FScopeLock Lock(&BuffersCritical);
		Buffer->Next = FirstFreeBuffer;
		FirstFreeBuffer = Buffer;
	}
	Stats.OnBufferReleased();
}

FFileIoStoreBlockCache::FFileIoStoreBlockCache(FFileIoStoreStats& InStats)
	: Stats(InStats)
{
	CacheLruHead.LruNext = &CacheLruTail;
	CacheLruTail.LruPrev = &CacheLruHead;
}

FFileIoStoreBlockCache::~FFileIoStoreBlockCache()
{
	FCachedBlock* CachedBlock = CacheLruHead.LruNext;
	while (CachedBlock != &CacheLruTail)
	{
		FCachedBlock* Next = CachedBlock->LruNext;
		delete CachedBlock;
		CachedBlock = Next;
	}
	FMemory::Free(CacheMemory);
}

void FFileIoStoreBlockCache::Initialize(uint64 InCacheMemorySize, uint64 InReadBufferSize)
{
	ReadBufferSize = InReadBufferSize;
	uint64 CacheBlockCount = InCacheMemorySize / InReadBufferSize;
	if (CacheBlockCount)
	{
		InCacheMemorySize = CacheBlockCount * InReadBufferSize;
		CacheMemory = reinterpret_cast<uint8*>(FMemory::Malloc(InCacheMemorySize));
		FCachedBlock* Prev = &CacheLruHead;
		for (uint64 CacheBlockIndex = 0; CacheBlockIndex < CacheBlockCount; ++CacheBlockIndex)
		{
			FCachedBlock* CachedBlock = new FCachedBlock();
			CachedBlock->Key = uint64(-1);
			CachedBlock->Buffer = CacheMemory + CacheBlockIndex * InReadBufferSize;
			Prev->LruNext = CachedBlock;
			CachedBlock->LruPrev = Prev;
			Prev = CachedBlock;
		}
		Prev->LruNext = &CacheLruTail;
		CacheLruTail.LruPrev = Prev;
	}
}

bool FFileIoStoreBlockCache::Read(FFileIoStoreReadRequest* Block)
{
	if (!CacheMemory)
	{
		return false;
	}
	check(Block->Buffer);
	FCachedBlock* CachedBlock = CachedBlocks.FindRef(Block->Key.Hash);
	if (!CachedBlock)
	{
		Stats.OnBlockCacheMiss(ReadBufferSize);
		return false;
	}
	
	CachedBlock->LruPrev->LruNext = CachedBlock->LruNext;
	CachedBlock->LruNext->LruPrev = CachedBlock->LruPrev;

	CachedBlock->LruPrev = &CacheLruHead;
	CachedBlock->LruNext = CacheLruHead.LruNext;

	CachedBlock->LruPrev->LruNext = CachedBlock;
	CachedBlock->LruNext->LruPrev = CachedBlock;

	check(CachedBlock->Buffer);
	Stats.OnBlockCacheHit(ReadBufferSize);
	FMemory::Memcpy(Block->Buffer->Memory, CachedBlock->Buffer, ReadBufferSize);

	return true;
}

void FFileIoStoreBlockCache::Store(const FFileIoStoreReadRequest* Block)
{
	bool bIsCacheableBlock = CacheMemory != nullptr && Block->BytesUsed < Block->Size;
	if (!bIsCacheableBlock)
	{
		return;
	}
	check(Block->Buffer);
	check(Block->Buffer->Memory);
	FCachedBlock* BlockToReplace = CacheLruTail.LruPrev;
	if (BlockToReplace == &CacheLruHead)
	{
		return;
	}
	check(BlockToReplace);
	CachedBlocks.Remove(BlockToReplace->Key);
	BlockToReplace->Key = Block->Key.Hash;

	BlockToReplace->LruPrev->LruNext = BlockToReplace->LruNext;
	BlockToReplace->LruNext->LruPrev = BlockToReplace->LruPrev;

	BlockToReplace->LruPrev = &CacheLruHead;
	BlockToReplace->LruNext = CacheLruHead.LruNext;

	BlockToReplace->LruPrev->LruNext = BlockToReplace;
	BlockToReplace->LruNext->LruPrev = BlockToReplace;
	
	check(BlockToReplace->Buffer);
	FMemory::Memcpy(BlockToReplace->Buffer, Block->Buffer->Memory, ReadBufferSize);
	Stats.OnBlockCacheStore(ReadBufferSize);
	CachedBlocks.Add(BlockToReplace->Key, BlockToReplace);
}

#define UE_FILEIOSTORE_DETAILED_QUEUE_COUNTERS_ENABLED 0
#if UE_FILEIOSTORE_DETAILED_QUEUE_COUNTERS_ENABLED
TRACE_DECLARE_INT_COUNTER(IoDispatcherLatencyCircuitBreaks, TEXT("IoDispatcher/LatencyCircuitBreaks"));
TRACE_DECLARE_INT_COUNTER(IoDispatcherSeekDistanceCircuitBreaks, TEXT("IoDispatcher/SeekDistanceCircuitBreaks"));
TRACE_DECLARE_INT_COUNTER(IoDispatcherNumPriorityQueues, TEXT("IoDispatcher/NumPriorityQueues"));
#endif

bool FFileIoStoreOffsetSortedRequestQueue::RequestSortPredicate(const FFileIoStoreReadRequestSortKey& A, const FFileIoStoreReadRequestSortKey& B)
{
	if (A.Handle == B.Handle)
	{
		return A.Offset < B.Offset;
	}
	return A.Handle < B.Handle;
}

FFileIoStoreOffsetSortedRequestQueue::FFileIoStoreOffsetSortedRequestQueue(int32 InPriority)
	: Priority(InPriority)
{
}

TArray<FFileIoStoreReadRequest*> FFileIoStoreOffsetSortedRequestQueue::StealRequests()
{
	RequestsBySequence.Clear();
	PeekRequestIndex = INDEX_NONE;
	return MoveTemp(Requests); 
}

// This could be potentially optimized if the higher level keeps track of which requests it changes the priority of, or even just the old priorty levels
TArray<FFileIoStoreReadRequest*> FFileIoStoreOffsetSortedRequestQueue::RemoveMisprioritizedRequests()
{
	PeekRequestIndex = INDEX_NONE;
	TArray<FFileIoStoreReadRequest*> RequestsToReturn;
	for (int32 i = Requests.Num()-1; i >= 0; --i)
	{
		if (Requests[i]->Priority != Priority)
		{
			RequestsToReturn.Add(Requests[i]);
			RequestsBySequence.Remove(Requests[i]);
			Requests.RemoveAt(i, EAllowShrinking::No);
		}
	}

	return RequestsToReturn;
}

void FFileIoStoreOffsetSortedRequestQueue::RemoveCancelledRequests(TArray<FFileIoStoreReadRequest*>& OutCancelled)
{
	for (int32 Idx = Requests.Num() - 1; Idx >= 0; --Idx)
	{
		FFileIoStoreReadRequest* Request = Requests[Idx];
		if (Request->bCancelled)
		{
			PeekRequestIndex = INDEX_NONE;
			OutCancelled.Add(Request);
			RequestsBySequence.Remove(Request);
			Requests.RemoveAt(Idx, EAllowShrinking::No);
		}
	}
}

FFileIoStoreReadRequest* FFileIoStoreOffsetSortedRequestQueue::GetNextInternal(FFileIoStoreReadRequestSortKey LastSortKey, bool bPop)
{
	if (Requests.Num() == 0)
	{
		return nullptr;
	}

	int32 RequestIndex = INDEX_NONE;
	if (PeekRequestIndex != INDEX_NONE)
	{
		RequestIndex = PeekRequestIndex;
	}
	else 
	{
		bool bHeadRequestTooOld = false;
		if (GIoDispatcherRequestLatencyCircuitBreakerMS > 0)
		{
			// If our oldest request has been unserviced for too long, grab that instead of the next sequential read
			uint64 ThresholdCycles = uint64((GIoDispatcherRequestLatencyCircuitBreakerMS * 1000.0) / FPlatformTime::GetSecondsPerCycle64());
			bHeadRequestTooOld = (FPlatformTime::Cycles64() - RequestsBySequence.PeekHead()->CreationTime) >= ThresholdCycles;

#if UE_FILEIOSTORE_DETAILED_QUEUE_COUNTERS_ENABLED
			if (bPop)
			{
				TRACE_COUNTER_INCREMENT(IoDispatcherLatencyCircuitBreaks);
			}
#endif
		}

		const bool bChooseByOffset = 
				LastSortKey.Handle != 0 
			&&	!bHeadRequestTooOld 
			&&  (GIoDispatcherMaintainSortingOnPriorityChange || LastSortKey.Priority == Priority);
		if (bChooseByOffset)
		{
			// Pick the request with the closest offset to the last thing that we read
			RequestIndex = Algo::LowerBoundBy(Requests, LastSortKey, RequestSortProjection, RequestSortPredicate);
			if (Requests.IsValidIndex(RequestIndex)) // If all requests are before LastOffset we get back out-of-bounds
			{
				if (Requests[RequestIndex]->ContainerFilePartition->FileHandle != LastSortKey.Handle)
				{
					// Changing file handle so switch back to the oldest outstanding request 
					RequestIndex = INDEX_NONE;
				}
				else if (GIoDispatcherMaxForwardSeekKB > 0 && (LastSortKey.Offset - Requests[RequestIndex]->Offset) > uint64(GIoDispatcherMaxForwardSeekKB) * 1024)
				{
					// Large forward seek so switch back to the oldest outstanding request 
					RequestIndex = INDEX_NONE;

#if UE_FILEIOSTORE_DETAILED_QUEUE_COUNTERS_ENABLED
					if (bPop)
					{
						TRACE_COUNTER_INCREMENT(IoDispatcherSeekDistanceCircuitBreaks);
					}
#endif
				}
			}
		}

		if (!Requests.IsValidIndex(RequestIndex))
		{
			RequestIndex = Requests.Find(RequestsBySequence.PeekHead());
			check(Requests[RequestIndex] == RequestsBySequence.PeekHead());
		}
	}

	check(Requests.IsValidIndex(RequestIndex));

	FFileIoStoreReadRequest* Request = Requests[RequestIndex];
	if (bPop)
	{
		Requests.RemoveAt(RequestIndex);
		RequestsBySequence.Remove(Request);
		PeekRequestIndex = INDEX_NONE;
	}
	else
	{
		PeekRequestIndex = RequestIndex;
	}
	return Request;
}

FFileIoStoreReadRequest* FFileIoStoreOffsetSortedRequestQueue::Pop(FFileIoStoreReadRequestSortKey LastSortKey)
{
	return GetNextInternal(LastSortKey, true);
}

void FFileIoStoreOffsetSortedRequestQueue::Push(FFileIoStoreReadRequest* Request)
{
	// Insert sorted by file handle & offset
	int32 InsertIndex = Algo::UpperBoundBy(Requests, RequestSortProjection(Request), RequestSortProjection, RequestSortPredicate);
	Requests.Insert(Request, InsertIndex);
	
	// Insert sorted by age
	RequestsBySequence.Add(Request);

	PeekRequestIndex = INDEX_NONE;
}

int32 HandleContainerUnmounted(const TArrayView<FFileIoStoreReadRequest*> Requests, const FFileIoStoreContainerFile& ContainerFile)
{
	static FFileIoStoreContainerFilePartition UnmountedPartition;

	int32 FailedRequestsCount = 0;

	for (const FFileIoStoreContainerFilePartition& Partition : ContainerFile.Partitions)
	{
		for (FFileIoStoreReadRequest* Request : Requests)
		{
			if (Request->ContainerFilePartition == &Partition)
			{
				Request->bFailed = true;
				Request->ContainerFilePartition = &UnmountedPartition;
				++FailedRequestsCount;
			}
		}
	}

	return FailedRequestsCount;
}

int32 FFileIoStoreOffsetSortedRequestQueue::HandleContainerUnmounted(const FFileIoStoreContainerFile& ContainerFile)
{
	return ::HandleContainerUnmounted(Requests, ContainerFile);
}

void FFileIoStoreRequestQueue::UpdateSortRequestsByOffset()
{
	// Must hold CriticalSection here
	if (bSortRequestsByOffset == bool(GIoDispatcherSortRequestsByOffset))
	{
		return;
	}

	bSortRequestsByOffset = bool(GIoDispatcherSortRequestsByOffset);
	if (bSortRequestsByOffset)
	{
		// Split things into separate heaps
		for (FFileIoStoreReadRequest* Request : Heap)
		{
			Push(*Request);
		}
		Heap.Empty();
	}
	else
	{
		// Put things back into the main heap
		TArray< FFileIoStoreReadRequest*> AllRequests;
		for (FFileIoStoreOffsetSortedRequestQueue& SubQueue : SortedPriorityQueues)
		{
			AllRequests.Append(SubQueue.StealRequests());
		}
		Algo::SortBy(AllRequests, [](FFileIoStoreReadRequest* Request) { return Request->Sequence; });
		for (FFileIoStoreReadRequest* Request : AllRequests)
		{
			Push(*Request);
		}
		check(Algo::AllOf(SortedPriorityQueues, [](const FFileIoStoreOffsetSortedRequestQueue& Q) { return Q.IsEmpty(); }));
		SortedPriorityQueues.Empty();
	}
}

FFileIoStoreReadRequest* FFileIoStoreRequestQueue::Pop()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(RequestQueuePop);
	FScopeLock _(&CriticalSection);
	UpdateSortRequestsByOffset();
	FFileIoStoreReadRequest* Result = nullptr;
	if (bSortRequestsByOffset)
	{
		if (SortedPriorityQueues.Num() == 0)
		{
			return nullptr;
		}

		FFileIoStoreOffsetSortedRequestQueue& SubQueue = SortedPriorityQueues.Last();
		check(!SubQueue.IsEmpty());
		Result = SubQueue.Pop(LastSortKey);
		check(Result);
		LastSortKey = Result;
		if (SubQueue.IsEmpty())
		{
			SortedPriorityQueues.Pop();
			// SubQueue is invalid here
#if UE_FILEIOSTORE_DETAILED_QUEUE_COUNTERS_ENABLED
			TRACE_COUNTER_DECREMENT(IoDispatcherNumPriorityQueues);
#endif
		}
	}
	else
	{
		if (Heap.Num() == 0)
		{
			return nullptr;
		}
		Heap.HeapPop(Result, QueueSortFunc, EAllowShrinking::No);
	}
	
	check(Result->QueueStatus == FFileIoStoreReadRequest::QueueStatus_InQueue);
	Result->QueueStatus = FFileIoStoreReadRequest::QueueStatus_Started;
	Result->ContainerFilePartition->StartedReadRequestsCount.fetch_add(1, std::memory_order_release);
	return Result;
}

void FFileIoStoreRequestQueue::PopCancelled(TArray<FFileIoStoreReadRequest*>& OutCancelled)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(RequestQueuePopCancelled);
	FScopeLock _(&CriticalSection);
	UpdateSortRequestsByOffset();
	
	if (bSortRequestsByOffset)
	{
		for (FFileIoStoreOffsetSortedRequestQueue& PrioQueue : SortedPriorityQueues)
		{
			PrioQueue.RemoveCancelledRequests(OutCancelled);
		}

		// Pop/Peek rely on empty queues being culled
		SortedPriorityQueues.RemoveAll([](FFileIoStoreOffsetSortedRequestQueue& SubQueue) { return SubQueue.IsEmpty(); });
	}
	else
	{
		for (int32 Idx = Heap.Num() - 1; Idx >= 0; --Idx)
		{
			FFileIoStoreReadRequest* Request = Heap[Idx];
			if (Request->bCancelled)
			{
				OutCancelled.Add(Request);
				Heap.RemoveAt(Idx, EAllowShrinking::No);
			}
		}

		if (!OutCancelled.IsEmpty())
		{
			Heap.Heapify(QueueSortFunc);
		}
	}

	for (FFileIoStoreReadRequest* Request : OutCancelled)
	{
		check(Request->bCancelled);
		Request->QueueStatus = FFileIoStoreReadRequest::QueueStatus_Started;
		Request->ContainerFilePartition->StartedReadRequestsCount.fetch_add(1, std::memory_order_release);
	}
}

void FFileIoStoreRequestQueue::PushToPriorityQueues(FFileIoStoreReadRequest* Request)
{
	int32 QueueIndex = Algo::LowerBoundBy(SortedPriorityQueues, Request->Priority, QueuePriorityProjection, TLess<int32>());
	if (!SortedPriorityQueues.IsValidIndex(QueueIndex) || SortedPriorityQueues[QueueIndex].GetPriority() != Request->Priority)
	{
		SortedPriorityQueues.Insert(FFileIoStoreOffsetSortedRequestQueue(Request->Priority), QueueIndex);
#if UE_FILEIOSTORE_DETAILED_QUEUE_COUNTERS_ENABLED
		TRACE_COUNTER_INCREMENT(IoDispatcherNumPriorityQueues);
#endif
	}
	check(Algo::IsSortedBy(SortedPriorityQueues, QueuePriorityProjection, TLess<int32>()));
	FFileIoStoreOffsetSortedRequestQueue& Queue = SortedPriorityQueues[QueueIndex];
	check(Queue.GetPriority() == Request->Priority);
	Queue.Push(Request);
}

void FFileIoStoreRequestQueue::Push(FFileIoStoreReadRequest& Request)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(RequestQueuePush);
	FScopeLock _(&CriticalSection);
	UpdateSortRequestsByOffset();
	
	check(Request.QueueStatus == FFileIoStoreReadRequest::QueueStatus_NotInQueue);
	Request.QueueStatus = FFileIoStoreReadRequest::QueueStatus_InQueue;

	if (bSortRequestsByOffset)
	{
		PushToPriorityQueues(&Request);
	}
	else
	{
		Heap.HeapPush(&Request, QueueSortFunc);
	}
}

void FFileIoStoreRequestQueue::Push(FFileIoStoreReadRequestList& Requests)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(RequestQueuePush);
	FScopeLock _(&CriticalSection);
	UpdateSortRequestsByOffset();

	for (auto It = Requests.Steal(); It; ++It)
	{
		check(It->QueueStatus == FFileIoStoreReadRequest::QueueStatus_NotInQueue);
		It->QueueStatus = FFileIoStoreReadRequest::QueueStatus_InQueue;

		if (bSortRequestsByOffset)
		{
			PushToPriorityQueues(*It);
		}
		else
		{
			Heap.HeapPush(*It, QueueSortFunc);
		}
	}
}

void FFileIoStoreRequestQueue::UpdateOrder()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(RequestQueueUpdateOrder);
	FScopeLock _(&CriticalSection);
	UpdateSortRequestsByOffset();
	if (bSortRequestsByOffset)
	{
		TArray<FFileIoStoreReadRequest*> Requests;
		for (FFileIoStoreOffsetSortedRequestQueue& SubQueue : SortedPriorityQueues)
		{
			TArray<FFileIoStoreReadRequest*> RequestsRemoved = SubQueue.RemoveMisprioritizedRequests();
			Requests.Append(RequestsRemoved);
		}

		// Pop/Peek rely on empty queues being culled
		SortedPriorityQueues.RemoveAll([](FFileIoStoreOffsetSortedRequestQueue& SubQueue) { return SubQueue.IsEmpty(); });

		Algo::SortBy(Requests, [](FFileIoStoreReadRequest* Request) { return Request->Sequence;  });
		for (FFileIoStoreReadRequest* Request : Requests)
		{
			PushToPriorityQueues(Request);
		}
	}
	else
	{
		Heap.Heapify(QueueSortFunc);
	}
}

void FFileIoStoreRequestQueue::Lock()
{
	CriticalSection.Lock();
}

void FFileIoStoreRequestQueue::Unlock()
{
	CriticalSection.Unlock();
}

int32 FFileIoStoreRequestQueue::HandleContainerUnmounted(const FFileIoStoreContainerFile& ContainerFile)
{
	FScopeLock _(&CriticalSection);
	
	int32 FailedRequestsCount = 0;

	if (bSortRequestsByOffset)
	{
		for (FFileIoStoreOffsetSortedRequestQueue& SubQueue : SortedPriorityQueues)
		{
			FailedRequestsCount += SubQueue.HandleContainerUnmounted(ContainerFile);
		}
	}
	else
	{
		FailedRequestsCount += ::HandleContainerUnmounted(Heap, ContainerFile);
	}

	return FailedRequestsCount;
}

FFileIoStoreContainerFilePartition::~FFileIoStoreContainerFilePartition()
{
	UE::TUniqueLock Lock(MappedFileHandleMutex);
	delete MappedFileHandle.load(std::memory_order_relaxed);
}

IMappedFileHandle* FFileIoStoreContainerFilePartition::BorrowMappedFileHandle()
{
	IMappedFileHandle* Handle = MappedFileHandle.load(std::memory_order_acquire);
	if (UNLIKELY(Handle == nullptr))
	{
		UE::TUniqueLock Lock(MappedFileHandleMutex);
		Handle = MappedFileHandle.load(std::memory_order_relaxed);
		if (UNLIKELY(Handle == nullptr))
		{
			IPlatformFile& Ipf = FPlatformFileManager::Get().GetPlatformFile();
			FOpenMappedResult Result = Ipf.OpenMappedEx2(*FilePath);
			if (!Result.HasError())
			{
				Handle = Result.StealValue().Release();
				MappedFileHandle.store(Handle, std::memory_order_release);
			}
			else
			{
				UE_LOGF(LogIoDispatcher, Error, "OpenMappedEx failed on: %ls", *FilePath);
			}
		}
	}
	return Handle;
}

FFileIoStoreReader::FFileIoStoreReader(IPlatformFileIoStore& InPlatformImpl, FFileIoStoreStats& InStats)
	: PlatformImpl(InPlatformImpl)
	, Stats(InStats)
{
}

FFileIoStoreReader::~FFileIoStoreReader()
{
	Close();
}

uint64 FFileIoStoreReader::GetTocAllocatedSize() const
{
	return DataContainer.GetAllocatedSize() + TocImperfectHashMapFallback.GetAllocatedSize();
}

FIoStatus FFileIoStoreReader::Initialize(const TCHAR* InTocFilePath, int32 InOrder)
{
	UE_TRACE_METADATA_SCOPE_ASSET_FNAME(FName(InTocFilePath), FName("FileIoStoreReader"), FName(InTocFilePath));

	FStringView ContainerPathView(InTocFilePath);
	if (!ContainerPathView.EndsWith(TEXT(".utoc")))
	{
		return FIoStatusBuilder(EIoErrorCode::FileOpenFailed) << TEXT("Expected .utoc extension on container path '") << InTocFilePath << TEXT("'");
	}
	FStringView BasePathView = ContainerPathView.LeftChop(5);
	ContainerFile.FilePath = BasePathView;

	IPlatformFile& Ipf = FPlatformFileManager::Get().GetPlatformFile();

	UE_LOGF(LogIoDispatcher, Verbose, "Reading toc: %ls", InTocFilePath);

	FIoStoreTocResourceView TocResource;
	FIoStoreTocResourceStorage TocStorage;
	FIoStatus Status = FIoStoreTocResourceView::Read(InTocFilePath, EIoStoreTocReadOptions::Default, TocResource, TocStorage);
	if (!Status.IsOk())
	{
		return Status;
	}

	ContainerFile.PartitionSize = TocResource.Header.PartitionSize;
	ContainerFile.Partitions.SetNum(TocResource.Header.PartitionCount);
	for (uint32 PartitionIndex = 0; PartitionIndex < TocResource.Header.PartitionCount; ++PartitionIndex)
	{
		FFileIoStoreContainerFilePartition& Partition = ContainerFile.Partitions[PartitionIndex];
		TStringBuilder<256> ContainerFilePath;
		ContainerFilePath.Append(BasePathView);
		if (PartitionIndex > 0)
		{
			ContainerFilePath.Appendf(TEXT("_s%d"), PartitionIndex);
		}
		ContainerFilePath.Append(TEXT(".ucas"));
		Partition.FilePath = ContainerFilePath;
		if (!PlatformImpl.OpenContainer(*ContainerFilePath, Partition.FileHandle, Partition.FileSize))
		{
			return FIoStatusBuilder(EIoErrorCode::FileOpenFailed) << TEXT("Failed to open IoStore container file '") << *ContainerFilePath << TEXT("'");
		}
		Partition.ContainerFileIndex = GlobalPartitionIndex++;
	}

	if (GIoDispatcherTocsEnablePerfectHashing && !TocResource.ChunkPerfectHashSeeds.IsEmpty())
	{
		for (int32 ChunkIndexWithoutPerfectHash : TocResource.ChunkIndicesWithoutPerfectHash)
		{
			TocImperfectHashMapFallback.Add(TocResource.ChunkIds[ChunkIndexWithoutPerfectHash], TocResource.ChunkOffsetLengths[ChunkIndexWithoutPerfectHash]);
		}

		PerfectHashMap.TocChunkHashSeeds = MoveTemp(TocResource.ChunkPerfectHashSeeds);
		PerfectHashMap.TocOffsetAndLengths = MoveTemp(TocResource.ChunkOffsetLengths);
		PerfectHashMap.TocChunkIds = MoveTemp(TocResource.ChunkIds);
		bHasPerfectHashMap = true;
	}
	else
	{
		UE_LOGF(LogIoDispatcher, Warning, "Falling back to imperfect hashmap for container '%ls'", InTocFilePath);
		for (uint32 ChunkIndex = 0; ChunkIndex < TocResource.Header.TocEntryCount; ++ChunkIndex)
		{
			TocImperfectHashMapFallback.Add(TocResource.ChunkIds[ChunkIndex], TocResource.ChunkOffsetLengths[ChunkIndex]);
		}
		bHasPerfectHashMap = false;
	}
	
	ContainerFile.CompressionMethods	= MoveTemp(TocResource.CompressionMethods);
	ContainerFile.CompressionBlockSize	= TocResource.Header.CompressionBlockSize;
	ContainerFile.CompressionBlocks		= MoveTemp(TocResource.CompressionBlocks);
	ContainerFile.ContainerFlags		= TocResource.Header.ContainerFlags;
	ContainerFile.EncryptionKeyGuid		= TocResource.Header.EncryptionKeyGuid;
	ContainerFile.BlockSignatureTable	= MoveTemp(TocResource.ChunkBlockSignatures);
	ContainerFile.ContainerInstanceId	= ++GlobalContainerInstanceId;

	Stats.OnTocMounted(GetTocAllocatedSize());

	UE_LOGF(LogIoDispatcher, Verbose, "Toc loaded : %ls, Id=%ls, Order=%d, EntryCount=%u, SignatureHash=%ls",
		InTocFilePath, *LexToString(TocResource.Header.ContainerId), InOrder, TocResource.Header.TocEntryCount, *TocResource.SignatureHash.ToString());

	// Hold onto the handle and mapped regions
	DataContainer = MoveTemp(TocStorage);
	ContainerId = TocResource.Header.ContainerId;
	Order = InOrder;
	return FIoStatus::Ok;
}

FIoStatus FFileIoStoreReader::Close()
{
	if (bClosed)
	{
		return FIoStatus::Ok;
	}

	for (FFileIoStoreContainerFilePartition& Partition : ContainerFile.Partitions)
	{
		PlatformImpl.CloseContainer(Partition.FileHandle);
	}

	Stats.OnTocUnmounted(GetTocAllocatedSize());

	PerfectHashMap = FFileIoStoreReader::FPerfectHashMap();
	TocImperfectHashMapFallback.Empty();
	ContainerFile.Empty();
	DataContainer = FIoStoreTocResourceStorage();
	ContainerId = FIoContainerId();
	Order = INDEX_NONE;
	bClosed = true;

	return FIoStatus::Ok;
}

const FIoOffsetAndLength* FFileIoStoreReader::FindChunkInternal(const FIoChunkId& ChunkId) const
{
	if (bHasPerfectHashMap)
	{
		// See FIoStoreWriterImpl::GeneratePerfectHashes
		const uint32 ChunkCount = PerfectHashMap.TocChunkIds.Num();
		if (!ChunkCount)
		{
			return nullptr;
		}
		const uint32 SeedCount = PerfectHashMap.TocChunkHashSeeds.Num();
		uint32 SeedIndex = FIoStoreTocResource::HashChunkIdWithSeed(0, ChunkId) % SeedCount;
		const int32 Seed = PerfectHashMap.TocChunkHashSeeds[SeedIndex];
		if (Seed == 0)
		{
			return nullptr;
		}
		uint32 Slot;
		if (Seed < 0)
		{
			uint32 SeedAsIndex = static_cast<uint32>(-Seed - 1);
			if (SeedAsIndex < ChunkCount)
			{
				Slot = static_cast<uint32>(SeedAsIndex);
			}
			else
			{
				// Entry without perfect hash
				return TocImperfectHashMapFallback.Find(ChunkId);
			}
		}
		else
		{
			Slot = FIoStoreTocResource::HashChunkIdWithSeed(static_cast<uint32>(Seed), ChunkId) % ChunkCount;
		}
		if (PerfectHashMap.TocChunkIds[Slot] == ChunkId)
		{
			return &PerfectHashMap.TocOffsetAndLengths[Slot];
		}
		return nullptr;
	}
	else
	{
		return TocImperfectHashMapFallback.Find(ChunkId);
	}
}

bool FFileIoStoreReader::DoesChunkExist(const FIoChunkId& ChunkId) const
{
	check(!bClosed);
	return FindChunkInternal(ChunkId) != nullptr;
}

TIoStatusOr<uint64> FFileIoStoreReader::GetSizeForChunk(const FIoChunkId& ChunkId) const
{
	check(!bClosed);
	const FIoOffsetAndLength* OffsetAndLength = FindChunkInternal(ChunkId);
	if (OffsetAndLength)
	{
		return OffsetAndLength->GetLength();
	}
	else
	{
		return FIoStatus(EIoErrorCode::NotFound);
	}
}

const FIoOffsetAndLength* FFileIoStoreReader::Resolve(const FIoChunkId& ChunkId) const
{
	check(!bClosed);
	return FindChunkInternal(ChunkId);
}

TUniquePtr<IMappedFileHandle> FFileIoStoreReader::GetMappedContainerFileHandle(uint64 TocOffset)
{
	check(!bClosed);
	int32 PartitionIndex = int32(TocOffset / ContainerFile.PartitionSize);
	FFileIoStoreContainerFilePartition& Partition = ContainerFile.Partitions[PartitionIndex];
	IMappedFileHandle* MappedFileHandle = nullptr;
	// Can't map encrypted files, compression should be disabled for bulk data when memory mapping is required
	if (!EnumHasAnyFlags(ContainerFile.ContainerFlags, EIoContainerFlags::Encrypted))
	{
		MappedFileHandle = Partition.BorrowMappedFileHandle();
	}

	check(Partition.FileSize > 0);
	return TUniquePtr<IMappedFileHandle>(new FMappedFileProxy(MappedFileHandle, Partition.FileSize, ContainerFile.PartitionSize));
}

TIoStatusOr<FIoContainerHeader> FFileIoStoreReader::ReadContainerHeader(bool bReadSoftRefs) const
{
	LLM_SCOPE_BYNAME(TEXT("Filesystem/FileIOStore"));
	TRACE_CPUPROFILER_EVENT_SCOPE(ReadContainerHeader);
	FIoChunkId HeaderChunkId = CreateIoChunkId(ContainerId.Value(), 0, EIoChunkType::ContainerHeader);
	const FIoOffsetAndLength* OffsetAndLength = FindChunkInternal(HeaderChunkId);
	if (!OffsetAndLength)
	{
		return FIoStatus(FIoStatusBuilder(EIoErrorCode::NotFound) << TEXT("Container header chunk not found"));
	}
	
	const uint64 CompressionBlockSize = ContainerFile.CompressionBlockSize;
	const uint64 Offset = OffsetAndLength->GetOffset();
	const uint64 Size = OffsetAndLength->GetLength();
	const uint64 RequestEndOffset = Offset + Size;
	const int32 RequestBeginBlockIndex = int32(Offset / CompressionBlockSize);
	const int32 RequestEndBlockIndex = int32((RequestEndOffset - 1) / CompressionBlockSize);
	
	// Assumes that the container header is uncompressed and placed in its own blocks in the same partition without padding
	const FIoStoreTocCompressedBlockEntry* CompressionBlockEntry = &ContainerFile.CompressionBlocks[RequestBeginBlockIndex];
	const int32 PartitionIndex = int32(CompressionBlockEntry->GetOffset() / ContainerFile.PartitionSize);
	const FFileIoStoreContainerFilePartition& Partition = ContainerFile.Partitions[PartitionIndex];
	const uint64 RawOffset = CompressionBlockEntry->GetOffset() % ContainerFile.PartitionSize;

#if !UE_BUILD_SHIPPING
	// Check for flag - compressed containers still have CompressionBlockSize != 0 and CompressionMethod "None".
	if (EnumHasAnyFlags(ContainerFile.ContainerFlags, EIoContainerFlags::Compressed)) 
	{
		FileCache_PostIoStoreCompressionBlockSize(IntCastChecked<int32>(CompressionBlockSize), Partition.FilePath);
	}
#endif

	FIoBuffer IoBuffer(Align(Size, FAES::AESBlockSize));
	IPlatformFile& Ipf = FPlatformFileManager::Get().GetPlatformFile();
	TUniquePtr<IFileHandle> ContainerFileHandle(Ipf.OpenRead(*Partition.FilePath));
	if (!ContainerFileHandle)
	{
		return FIoStatus(FIoStatusBuilder(EIoErrorCode::FileOpenFailed) << TEXT("Failed to open container file ") << Partition.FilePath);
	}
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ReadFromContainerFile);
		if (!ContainerFileHandle->Seek(RawOffset))
		{
			return FIoStatus(FIoStatusBuilder(EIoErrorCode::ReadError) << FString::Printf(TEXT("Failed seeking to offset %llu in container file"), RawOffset));
		}
		if (!ContainerFileHandle->Read(IoBuffer.Data(), IoBuffer.DataSize()))
		{
			return FIoStatus(FIoStatusBuilder(EIoErrorCode::ReadError) << FString::Printf(TEXT("Failed reading %llu bytes at offset %llu"), IoBuffer.DataSize(), RawOffset));
		}
	}

	const bool bSigned = EnumHasAnyFlags(ContainerFile.ContainerFlags, EIoContainerFlags::Signed);
	const bool bEncrypted = ContainerFile.EncryptionKey.IsValid();
	if (bSigned || bEncrypted)
	{
		uint8* BlockData = IoBuffer.Data();
		for (int32 CompressedBlockIndex = RequestBeginBlockIndex; CompressedBlockIndex <= RequestEndBlockIndex; ++CompressedBlockIndex)
		{
			CompressionBlockEntry = &ContainerFile.CompressionBlocks[CompressedBlockIndex];
			check(ContainerFile.CompressionMethods[CompressionBlockEntry->GetCompressionMethodIndex()].IsNone());
			const uint64 BlockSize = Align(CompressionBlockEntry->GetCompressedSize(), FAES::AESBlockSize);
			if (bSigned)
			{
				const FSHAHash& SignatureHash = ContainerFile.BlockSignatureTable[CompressedBlockIndex];
				FSHAHash BlockHash;
				FSHA1::HashBuffer(BlockData, BlockSize, BlockHash.Hash);
				if (SignatureHash != BlockHash)
				{
					return FIoStatus(FIoStatusBuilder(EIoErrorCode::SignatureError) << TEXT("Signature error detected when reading container header"));
				}
			}
			if (bEncrypted)
			{
				FAES::DecryptData(BlockData, uint32(BlockSize), ContainerFile.EncryptionKey);
			}
			BlockData += BlockSize;
		}
	}
	FMemoryReaderView Ar(MakeArrayView(IoBuffer.Data(), static_cast<int32>(IoBuffer.DataSize())));
	FIoContainerHeader ContainerHeader;
	Ar << ContainerHeader;
	if (Ar.IsError())
	{
		UE_LOGF(LogIoDispatcher, Warning, "Invalid container header in file '%ls'", *ContainerFile.FilePath);
		ContainerHeader = FIoContainerHeader();
	}

	// todo, ensure here if the container didn't include softreferences and we want to load them, the size check isn't really sufficent
	if (bReadSoftRefs)
	{
		if (ContainerHeader.SoftPackageReferencesSerialInfo.Offset < 0)
		{
			FIoStatus Status = FIoStatusBuilder(EIoErrorCode::ReadError)
				<< FString::Printf(TEXT("Invalid soft package reference offset '%" INT64_FMT "'"), ContainerHeader.SoftPackageReferencesSerialInfo.Offset);
			return Status;
		}
		if ((ContainerHeader.SoftPackageReferencesSerialInfo.Offset + ContainerHeader.SoftPackageReferencesSerialInfo.Size) > Ar.TotalSize())
		{
			FIoStatus Status = FIoStatusBuilder(EIoErrorCode::ReadError)
				<< FString::Printf(TEXT("Soft package reference offset '%" INT64_FMT "' and size '%" INT64_FMT "' will seek past the end of archive size '%" INT64_FMT "'"),
					ContainerHeader.SoftPackageReferencesSerialInfo.Offset,
					ContainerHeader.SoftPackageReferencesSerialInfo.Size,
					Ar.TotalSize());
			return Status;
		}
		Ar.Seek(ContainerHeader.SoftPackageReferencesSerialInfo.Offset);
		Ar << ContainerHeader.SoftPackageReferences;
	}

	return ContainerHeader;
}

void FFileIoStoreReader::ReopenAllFileHandles()
{
	for (FFileIoStoreContainerFilePartition& Partition : ContainerFile.Partitions)
	{
		PlatformImpl.CloseContainer(Partition.FileHandle);
		PlatformImpl.OpenContainer(*Partition.FilePath, Partition.FileHandle, Partition.FileSize);
	}
}

FFileIoStoreResolvedRequest::FFileIoStoreResolvedRequest(
	FIoRequestImpl& InDispatcherRequest,
	FFileIoStoreContainerFile* InContainerFile,
	uint64 InResolvedOffset,
	uint64 InResolvedSize,
	int32 InPriority)
	: DispatcherRequest(&InDispatcherRequest)
	, ContainerFile(InContainerFile)
	, ResolvedOffset(InResolvedOffset)
	, ResolvedSize(InResolvedSize)
	, Priority(InPriority)
{
	ContainerFile->RefCount.fetch_add(1, std::memory_order_acq_rel);
}

void FFileIoStoreResolvedRequest::AddReadRequestLink(FFileIoStoreReadRequestLink* ReadRequestLink)
{
	check(!ReadRequestLink->Next);
	if (ReadRequestsTail)
	{
		ReadRequestsTail->Next = ReadRequestLink;
	}
	else
	{
		ReadRequestsHead = ReadRequestLink;
	}
	ReadRequestsTail = ReadRequestLink;
}

FFileIoStoreRequestTracker::FFileIoStoreRequestTracker(FFileIoStoreRequestAllocator& InRequestAllocator, FFileIoStoreRequestQueue& InRequestQueue)
	: RequestAllocator(InRequestAllocator)
	, RequestQueue(InRequestQueue)
{

}

FFileIoStoreRequestTracker::~FFileIoStoreRequestTracker()
{

}

FFileIoStoreCompressedBlock* FFileIoStoreRequestTracker::FindOrAddCompressedBlock(FFileIoStoreBlockKey Key, bool& bOutWasAdded)
{
	bOutWasAdded = false;
	FFileIoStoreCompressedBlock*& Result = CompressedBlocksMap.FindOrAdd(Key);
	if (!Result)
	{
		Result = RequestAllocator.AllocCompressedBlock();
		Result->Key = Key;
		bOutWasAdded = true;
	}
	return Result;
}

FFileIoStoreReadRequest* FFileIoStoreRequestTracker::FindOrAddRawBlock(FFileIoStoreBlockKey Key, bool& bOutWasAdded)
{
	bOutWasAdded = false;
	FFileIoStoreReadRequest*& Result = RawBlocksMap.FindOrAdd(Key);
	if (!Result)
	{
		Result = RequestAllocator.AllocReadRequest();
		Result->Key = Key;
		bOutWasAdded = true;
	}
	return Result;
}

void FFileIoStoreRequestTracker::RemoveRawBlock(const FFileIoStoreReadRequest* RawBlock, bool bRemoveFromCancel)
{
	if (!RawBlock->bCancelled || bRemoveFromCancel)
	{
		RawBlocksMap.Remove(RawBlock->Key);
		if (RawBlocksMap.IsEmpty())
		{
			RawBlocksMap.Empty(128);
		}
	}
}

void FFileIoStoreRequestTracker::AddReadRequestsToResolvedRequest(FFileIoStoreCompressedBlock* CompressedBlock, FFileIoStoreResolvedRequest& ResolvedRequest)
{
	//TRACE_CPUPROFILER_EVENT_SCOPE(AddReadRequestsToResolvedRequest);
	bool bUpdateQueueOrder = false;
	++ResolvedRequest.UnfinishedReadsCount;
	for (FFileIoStoreReadRequest* ReadRequest : CompressedBlock->RawBlocks)
	{
		FFileIoStoreReadRequestLink* Link = RequestAllocator.AllocRequestLink(ReadRequest);
		++ReadRequest->RefCount;
		ResolvedRequest.AddReadRequestLink(Link);
		if (ResolvedRequest.GetPriority() > ReadRequest->Priority)
		{
			ReadRequest->Priority = ResolvedRequest.GetPriority();
			bUpdateQueueOrder = true;
		}
	}
	if (bUpdateQueueOrder)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(RequestTrackerAddIoRequestUpdateOrder);
		RequestQueue.UpdateOrder();
	}
}

void FFileIoStoreRequestTracker::AddReadRequestsToResolvedRequest(const FFileIoStoreReadRequestList& Requests, FFileIoStoreResolvedRequest& ResolvedRequest)
{
	//TRACE_CPUPROFILER_EVENT_SCOPE(RequestTrackerAddIoRequest);
	for (FFileIoStoreReadRequest* Request : Requests)
	{
		++ResolvedRequest.UnfinishedReadsCount;
		FFileIoStoreReadRequestLink* Link = RequestAllocator.AllocRequestLink(Request);
		++Request->RefCount;
		ResolvedRequest.AddReadRequestLink(Link);
		check(ResolvedRequest.GetPriority() == Request->Priority);
	}
}

void FFileIoStoreRequestTracker::RemoveCompressedBlock(const FFileIoStoreCompressedBlock* CompressedBlock, bool bRemoveFromCancel)
{
	if (!CompressedBlock->bCancelled || bRemoveFromCancel)
	{
		CompressedBlocksMap.Remove(CompressedBlock->Key);
		if (CompressedBlocksMap.IsEmpty())
		{
			CompressedBlocksMap.Empty(512);
		}
	}
}

bool FFileIoStoreRequestTracker::CancelIoRequest(FFileIoStoreResolvedRequest& ResolvedRequest)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(RequestTrackerCancelIoRequest);
	check(!ResolvedRequest.bCancelled);
	bool bShouldComplete = true;
	RequestQueue.Lock();
	FFileIoStoreReadRequestLink* Link = ResolvedRequest.ReadRequestsHead;
	while (Link)
	{
		FFileIoStoreReadRequest& ReadRequest = Link->ReadRequest;
		Link = Link->Next;

		if (ReadRequest.bCancelled)
		{
			continue;
		}

		if (ReadRequest.QueueStatus >= FFileIoStoreReadRequest::QueueStatus_Started)
		{
			bShouldComplete = false;
			continue;
		}

		bool bCancelReadRequest = true;
		for (FFileIoStoreCompressedBlock* CompressedBlock : ReadRequest.CompressedBlocks)
		{
			if (CompressedBlock->bCancelled)
			{
				continue;
			}
			bool bCancelCompressedBlock = true;
			for (FFileIoStoreBlockScatter& Scatter : CompressedBlock->ScatterList)
			{
				if (Scatter.Size > 0 && Scatter.Request != &ResolvedRequest)
				{
					bCancelCompressedBlock = false;
					bCancelReadRequest = false;
				}
				else
				{
					Scatter.Size = 0;
				}
			}
			if (bCancelCompressedBlock)
			{
				CompressedBlock->bCancelled = true;
				RemoveCompressedBlock(CompressedBlock, /*bRemoveFromCancel*/ true);
			}
		}
		if (bCancelReadRequest)
		{
			ReadRequest.bCancelled = true;
			if (!ReadRequest.ImmediateScatter.Request)
			{
				RemoveRawBlock(&ReadRequest, /*bRemoveFromCancel*/ true);
			}
#if DO_CHECK
			for (FFileIoStoreCompressedBlock* CompressedBlock : ReadRequest.CompressedBlocks)
			{
				check(CompressedBlock->bCancelled);
				for (FFileIoStoreBlockScatter& Scatter : CompressedBlock->ScatterList)
				{
					check(!Scatter.Request->DispatcherRequest || Scatter.Request->DispatcherRequest->IsCancelled());
				}
			}
#endif
		}
	}
	RequestQueue.Unlock();

	return bShouldComplete;
}

void FFileIoStoreRequestTracker::UpdatePriorityForIoRequest(FFileIoStoreResolvedRequest& ResolvedRequest)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(RequestTrackerUpdatePriorityForIoRequest);
	bool bUpdateOrder = false;
	FFileIoStoreReadRequestLink* Link = ResolvedRequest.ReadRequestsHead;
	while (Link)
	{
		FFileIoStoreReadRequest& ReadRequest = Link->ReadRequest;
		Link = Link->Next;
		if (ResolvedRequest.GetPriority() > ReadRequest.Priority)
		{
			ReadRequest.Priority = ResolvedRequest.GetPriority();
			bUpdateOrder = true;
		}
	}
	if (bUpdateOrder)
	{
		RequestQueue.UpdateOrder();
	}
}

void FFileIoStoreRequestTracker::ReleaseIoRequestReferences(FFileIoStoreResolvedRequest& ResolvedRequest)
{
	FFileIoStoreReadRequestLink* Link = ResolvedRequest.ReadRequestsHead;
	while (Link)
	{
		FFileIoStoreReadRequestLink* Next = Link->Next;
		check(Link->ReadRequest.RefCount > 0);
		if (--Link->ReadRequest.RefCount == 0)
		{
			for (FFileIoStoreCompressedBlock* CompressedBlock : Link->ReadRequest.CompressedBlocks)
			{
				check(CompressedBlock->RefCount > 0);
				if (--CompressedBlock->RefCount == 0)
				{
					RequestAllocator.Free(CompressedBlock);
				}
			}
			RequestAllocator.Free(&Link->ReadRequest);
		}
		RequestAllocator.Free(Link);
		Link = Next;
	}
	ResolvedRequest.ReadRequestsHead = nullptr;
	ResolvedRequest.ReadRequestsTail = nullptr;
	RequestAllocator.Free(&ResolvedRequest);
}

int64 FFileIoStoreRequestTracker::GetLiveReadRequestsCount() const
{
	return RequestAllocator.GetLiveReadRequestsCount();
}

FFileIoStoreReadersToUnload::~FFileIoStoreReadersToUnload()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(WaitOnReferencedContainers);

	const double MaxWaitTime = FMath::Clamp(GFileIoStoreUnmountTimeoutSeconds, 1.0, 10.0);
	const double StartTime = FPlatformTime::Seconds();

	for (;;)
	{
		int32 Pending = 0;
		{
			UE::TUniqueLock Lock(Mutex);
			for (TUniquePtr<FFileIoStoreReader>& Reader : Readers)
			{
				if (Reader->GetContainerFile()->RefCount.load(std::memory_order_acquire) > 0)
				{
					Pending++;
				}
			}
		}

		if (Pending == 0)
		{
			break;
		}

		if (double WaitTime = FPlatformTime::Seconds() - StartTime; WaitTime > MaxWaitTime)
		{
			UE_LOGF(LogIoDispatcher, Error, "Shutting down file I/O dispatcher backend with %d referenced container(s).", Pending);
			break;
		}

		FPlatformProcess::SleepNoStats(0.01f);
	}
}

void FFileIoStoreReadersToUnload::Add(TUniquePtr<FFileIoStoreReader>&& Reader)
{
	UE::TUniqueLock Lock(Mutex);
	Readers.Add(MoveTemp(Reader));
}

void FFileIoStoreReadersToUnload::TryUnload()
{
	TArray<TUniquePtr<FFileIoStoreReader>> ToUnload;
	{
		UE::TUniqueLock Lock(Mutex);
		for (auto It = Readers.CreateIterator(); It; ++It)
		{
			TUniquePtr<FFileIoStoreReader>& Reader	= *It;
			FFileIoStoreContainerFile& Container	= *Reader->GetContainerFile();
			const int32 RefCount					= Container.RefCount.load(std::memory_order_acquire);

			if (RefCount > 0)
			{
				UE_LOGF(LogIoDispatcher, Warning, "Waiting on %d pending I/O request before unloading container '%ls'",
					RefCount, *Container.FilePath);
			}
			else
			{
				check(RefCount == 0);
				ToUnload.Add(MoveTemp(Reader));
				It.RemoveCurrent();
			}
		}
	}

	for (TUniquePtr<FFileIoStoreReader>& Reader : ToUnload)
	{
		UE_LOGF(LogIoDispatcher, Display, "Unloading container '%ls'", *Reader->GetContainerFile()->FilePath);
	}
}

FFileIoStore::FFileIoStore(TUniquePtr<IPlatformFileIoStore>&& InPlatformImpl)
	: BlockCache(Stats)
	, BufferAllocator(Stats)
	, RequestTracker(RequestAllocator, RequestQueue)
	, PlatformImpl(MoveTemp(InPlatformImpl))
{
}

FFileIoStore::~FFileIoStore()
{
	StopThread();
}

void FFileIoStore::Initialize(TSharedRef<const FIoDispatcherBackendContext> InContext)
{
	check(!Thread);

	BackendContext = InContext;
	bIsMultithreaded = InContext->bIsMultiThreaded;

	ReadBufferSize = (GIoDispatcherBufferSizeKB > 0 ? uint64(GIoDispatcherBufferSizeKB) << 10 : 256 << 10);

	uint64 BufferMemorySize = uint64(GIoDispatcherBufferMemoryMB) << 20ull;
	uint64 BufferSize = uint64(GIoDispatcherBufferSizeKB) << 10ull;
	uint32 BufferAlignment = uint32(GIoDispatcherBufferAlignment);
	BufferAllocator.Initialize(BufferMemorySize, BufferSize, BufferAlignment);

	uint64 CacheMemorySize = uint64(GIoDispatcherCacheSizeMB) << 20ull;
	BlockCache.Initialize(CacheMemorySize, BufferSize);

	PlatformImpl->Initialize({
		&BackendContext->WakeUpDispatcherThreadDelegate,
		&RequestAllocator,
		&BufferAllocator,
		&BlockCache,
		&Stats
	});

	int32 DecompressionContextCount = int32(GIoDispatcherDecompressionWorkerCount > 0 ? GIoDispatcherDecompressionWorkerCount : 4);
	CompressionContexts.SetNum(DecompressionContextCount);
	for (TUniquePtr<FFileIoStoreCompressionContext>& CompressionContext : CompressionContexts)
	{
		CompressionContext = MakeUnique<FFileIoStoreCompressionContext>();
		CompressionContext->Next = FirstFreeCompressionContext;
		FirstFreeCompressionContext = CompressionContext.Get();
	}

	Thread = FRunnableThread::Create(this, TEXT("IoService"), 0, TPri_AboveNormal);

	using namespace LowLevelTasks;
	OversubscriptionLimitReached = 
		FScheduler::Get().GetOversubscriptionLimitReachedEvent().AddLambda(
			[this]()
			{
				BackendContext->WakeUpDispatcherThreadDelegate.Execute();
			}
		);
}

void FFileIoStore::StopThread()
{
	if (Thread)
	{
		delete Thread;
		Thread = nullptr;
	}
}

void FFileIoStore::Shutdown()
{
	StopThread();

	using namespace LowLevelTasks;
	FScheduler::Get().GetOversubscriptionLimitReachedEvent().Remove(OversubscriptionLimitReached);
}

TIoStatusOr<FIoContainerHeader> FFileIoStore::Mount(
	const TCHAR* InTocPath, 
	int32 Order, 
	const FGuid& EncryptionKeyGuid, 
	const FAES::FAESKey& EncryptionKey,
	UE::IoStore::ETocMountOptions Options)
{
	LLM_SCOPE_BYNAME(TEXT("FileSystem/FileIoStore"));
	TUniquePtr<FFileIoStoreReader> Reader(new FFileIoStoreReader(*PlatformImpl, Stats));
	FIoStatus IoStatus = Reader->Initialize(InTocPath, Order);
	if (!IoStatus.IsOk())
	{
		return IoStatus;
	}

	if (Reader->IsEncrypted())
	{
		if (Reader->GetEncryptionKeyGuid() == EncryptionKeyGuid && EncryptionKey.IsValid())
		{
			Reader->SetEncryptionKey(EncryptionKey);
		}
		else
		{
			return FIoStatus(EIoErrorCode::InvalidEncryptionKey, *FString::Printf(TEXT("Invalid encryption key '%s' (container '%s', encryption key '%s')"),
				*EncryptionKeyGuid.ToString(), InTocPath, *Reader->GetEncryptionKeyGuid().ToString()));
		}
	}

	TIoStatusOr<FIoContainerHeader> ContainerHeaderReadResult = Reader->ReadContainerHeader(
		EnumHasAnyFlags(Options, UE::IoStore::ETocMountOptions::WithSoftReferences));
	FIoContainerHeader ContainerHeader;
	if (ContainerHeaderReadResult.IsOk())
	{
		ContainerHeader = ContainerHeaderReadResult.ConsumeValueOrDie();
	}
	else if (EIoErrorCode::NotFound != ContainerHeaderReadResult.Status().GetErrorCode())
	{
		return ContainerHeaderReadResult;
	}

	int32 InsertionIndex;
	{
		FWriteScopeLock _(IoStoreReadersLock);
		InsertionIndex = Algo::UpperBound(IoStoreReaders, Reader, [](const TUniquePtr<FFileIoStoreReader>& A, const TUniquePtr<FFileIoStoreReader>& B)
		{
			if (A->GetOrder() != B->GetOrder())
			{
				return A->GetOrder() > B->GetOrder();
			}
			return A->GetContainerInstanceId() > B->GetContainerInstanceId();
		});
		const EIoContainerFlags ContainerFlags = Reader->GetContainerFile() != nullptr ? Reader->GetContainerFile()->ContainerFlags : EIoContainerFlags::None;
		IoStoreReaders.Insert(MoveTemp(Reader), InsertionIndex);
		UE_LOGF(LogIoDispatcher, Display, "Mounted container '%ls', Id='%ls', Order=%d, Slot=%d, Options=(%ls), Flags=(%ls), HasSoftRefs=%ls",
			InTocPath, *LexToString(ContainerHeader.ContainerId), Order, InsertionIndex, LexToString(Options), *LexToString(ContainerFlags),
			ContainerHeader.SoftPackageReferences.bContainsSoftPackageReferences ? TEXT("True") : TEXT("False"));
	}

	return ContainerHeader;
}

bool FFileIoStore::Unmount(const TCHAR* InTocPath)
{
	TUniquePtr<FFileIoStoreReader> ReaderToUnmount;
	{
		FWriteScopeLock _(IoStoreReadersLock);

		for (int32 Idx = 0; Idx < IoStoreReaders.Num(); ++Idx)
		{
			if (IoStoreReaders[Idx]->GetContainerFile()->FilePath == InTocPath)
			{
				ReaderToUnmount = MoveTemp(IoStoreReaders[Idx]);
				IoStoreReaders.RemoveAt(Idx);

				break;
			}
		}
	}
	if (ReaderToUnmount)
	{
		UE_LOGF(LogIoDispatcher, Display, "Unmounting container '%ls'", InTocPath);

		int32 FailedRequestsCount = RequestQueue.HandleContainerUnmounted(*ReaderToUnmount->GetContainerFile());

		UE_CLOGF(FailedRequestsCount > 0, LogIoDispatcher, Warning, "Marking %d queued requests from unmounted container as failed", FailedRequestsCount);

		if (GFileIoStoreAsyncUnmountEnabled)
		{
			IoStoreReadersToUnload.Add(MoveTemp(ReaderToUnmount));
			BackendContext->WakeUpDispatcherThreadDelegate.Execute();
			return true;
		}

		bool bHasWarned = false;
		for (const FFileIoStoreContainerFilePartition& Partition : ReaderToUnmount->GetContainerFile()->Partitions)
		{
			if (Partition.StartedReadRequestsCount.load(std::memory_order_acquire) != 0)
			{
				UE_CLOGF(!bHasWarned, LogIoDispatcher, Warning, "Waiting for read requests to finish before unmounting container");
				bHasWarned = true;
				while (Partition.StartedReadRequestsCount.load(std::memory_order_acquire) != 0)
				{
					FPlatformProcess::Sleep(0);
				}
			}
		}

		// Perform a second reference count check that also includes pending scatter tasks
		FFileIoStoreContainerFile& Container = *ReaderToUnmount->GetContainerFile();
		int32 RefCount = Container.RefCount.load(std::memory_order_acquire);
		UE_CLOGF(RefCount > 0, LogIoDispatcher, Warning, "Waiting for %d request(s) before unmounting container '%ls'",
			RefCount, InTocPath);

		const double MaxWaitTime = FMath::Clamp(GFileIoStoreUnmountTimeoutSeconds, 1.0, 10.0);
		const double StartTime = FPlatformTime::Seconds();
		while (RefCount > 0)
		{
			if (FPlatformTime::Seconds() - StartTime > MaxWaitTime)
			{
				UE_LOGF(LogIoDispatcher, Warning, "Unmounting container '%ls' with %d remaining requests(s) after waiting for %.2f seconds(s).",
					InTocPath, RefCount, GFileIoStoreUnmountTimeoutSeconds);
				break;
			}
			FPlatformProcess::SleepNoStats(0.0);
			RefCount = Container.RefCount.load(std::memory_order_acquire);
		}

		return true;
	}
	else
	{
		UE_LOGF(LogIoDispatcher, Display, "Failed to unmount container '%ls'", InTocPath);
	}
	
	return false;
}

bool FFileIoStore::Resolve(FIoRequestImpl* Request)
{
	// Assumes readers are locked, see ResolveIoRequests
	for (const TUniquePtr<FFileIoStoreReader>& Reader : IoStoreReaders)
	{
		if (const FIoOffsetAndLength* OffsetAndLength = Reader->Resolve(Request->ChunkId))
		{
			uint64 RequestedOffset = Request->Options.GetOffset();
			uint64 ResolvedOffset = OffsetAndLength->GetOffset() + RequestedOffset;
			uint64 ResolvedSize = 0;
			if (RequestedOffset <= OffsetAndLength->GetLength())
			{
				ResolvedSize = FMath::Min(Request->Options.GetSize(), OffsetAndLength->GetLength() - RequestedOffset);
			}

			FFileIoStoreResolvedRequest* ResolvedRequest = RequestAllocator.AllocResolvedRequest(
				*Request,
				Reader->GetContainerFile(),
				ResolvedOffset,
				ResolvedSize,
				Request->Priority);
			Request->BackendData = ResolvedRequest;

			TRACE_IOSTORE_BACKEND_REQUEST_STARTED(Request, this);
			if (ResolvedSize > 0)
			{
				FFileIoStoreReadRequestList CustomRequests;
				if (PlatformImpl->CreateCustomRequests(*ResolvedRequest, CustomRequests))
				{
					Stats.OnReadRequestsQueued(CustomRequests);
					RequestTracker.AddReadRequestsToResolvedRequest(CustomRequests, *ResolvedRequest);
					RequestQueue.Push(CustomRequests);
					OnNewPendingRequestsAdded();
				}
				else
				{
					ReadBlocks(*ResolvedRequest);
				}
			}
			else
			{
				// Nothing to read
				if (RequestedOffset > OffsetAndLength->GetLength())
				{
					ResolvedRequest->bFailed = true;
				}
				else
				{
					ResolvedRequest->CreateBuffer(0);
				}
				CompleteDispatcherRequest(ResolvedRequest);
				RequestTracker.ReleaseIoRequestReferences(*ResolvedRequest);
			}

			return true;
		}
	}

	return false;
}

void FFileIoStore::ResolveIoRequests(FIoRequestList Requests, FIoRequestList& OutUnresolved)
{
	FReadScopeLock _(IoStoreReadersLock);
	while (FIoRequestImpl* Request = Requests.PopHead())
	{
		if (Resolve(Request) == false)
		{
			OutUnresolved.AddTail(Request);
		}
	}
}

void FFileIoStore::CancelIoRequest(FIoRequestImpl* Request)
{
	if (Request->BackendData)
	{
		FFileIoStoreResolvedRequest* ResolvedRequest = static_cast<FFileIoStoreResolvedRequest*>(Request->BackendData);
		bool bShouldComplete = RequestTracker.CancelIoRequest(*ResolvedRequest);
		if (bShouldComplete)
		{
			ResolvedRequest->bCancelled = true;
			CompleteDispatcherRequest(ResolvedRequest);
		}
		else
		{
			// Wake-up the I/O thread to process cancelled read requests
			PlatformImpl->ServiceNotify();
		}
	}
}

void FFileIoStore::UpdatePriorityForIoRequest(FIoRequestImpl* Request)
{
	if (Request->BackendData)
	{
		FFileIoStoreResolvedRequest* ResolvedRequest = static_cast<FFileIoStoreResolvedRequest*>(Request->BackendData);
		ResolvedRequest->Priority = Request->Priority;
		RequestTracker.UpdatePriorityForIoRequest(*ResolvedRequest);
	}
}

bool FFileIoStore::DoesChunkExist(const FIoChunkId& ChunkId) const
{
	FReadScopeLock _(IoStoreReadersLock);
	for (const TUniquePtr<FFileIoStoreReader>& Reader : IoStoreReaders)
	{
		if (Reader->DoesChunkExist(ChunkId))
		{
			return true;
		}
	}
	return false;
}

TIoStatusOr<uint64> FFileIoStore::GetSizeForChunk(const FIoChunkId& ChunkId) const
{
	FReadScopeLock _(IoStoreReadersLock);
	for (const TUniquePtr<FFileIoStoreReader>& Reader : IoStoreReaders)
	{
		TIoStatusOr<uint64> ReaderResult = Reader->GetSizeForChunk(ChunkId);
		if (ReaderResult.IsOk())
		{
			return ReaderResult;
		}
	}
	return FIoStatus(EIoErrorCode::NotFound);
}

FAutoConsoleTaskPriority CPrio_IoDispatcherTaskPriority(
	TEXT("TaskGraph.TaskPriorities.IoDispatcherAsyncTasks"),
	TEXT("Task and thread priority for IoDispatcher decompression."),
	ENamedThreads::BackgroundThreadPriority, // if we have background priority task threads, then use them...
	ENamedThreads::NormalTaskPriority, // .. at normal task priority
	ENamedThreads::NormalTaskPriority // if we don't have background threads, then use normal priority threads at normal task priority instead
);

void FFileIoStore::ScatterBlock(FFileIoStoreCompressedBlock* CompressedBlock, bool bIsAsync)
{
	LLM_SCOPE_BYNAME(TEXT("FileSystem/FileIoStore"));
	TRACE_CPUPROFILER_EVENT_SCOPE(IoDispatcherScatter);
	
	check(!CompressedBlock->bFailed);

	FFileIoStoreCompressionContext* CompressionContext = CompressedBlock->CompressionContext;
	check(CompressionContext);
	uint8* CompressedBuffer;
	if (CompressedBlock->RawBlocks.Num() > 1)
	{
		check(CompressedBlock->CompressedDataBuffer);
		CompressedBuffer = CompressedBlock->CompressedDataBuffer;
	}
	else
	{
		FFileIoStoreReadRequest* RawBlock = CompressedBlock->RawBlocks[0];
		check(CompressedBlock->RawOffset >= RawBlock->Offset);
		uint64 OffsetInBuffer = CompressedBlock->RawOffset - RawBlock->Offset;
		CompressedBuffer = RawBlock->Buffer->Memory + OffsetInBuffer;
	}
	if (!CompressedBlock->bFailed)
	{
		if (CompressedBlock->SignatureHash)
		{
			FSHAHash BlockHash;
			FSHA1::HashBuffer(CompressedBuffer, CompressedBlock->RawSize, BlockHash.Hash);
			if (*CompressedBlock->SignatureHash != BlockHash)
			{
				FIoSignatureError Error;
				{
					FReadScopeLock _(IoStoreReadersLock);
					for (const TUniquePtr<FFileIoStoreReader>& Reader : IoStoreReaders)
					{
						if (CompressedBlock->Key.FileIndex == Reader->GetContainerInstanceId())
						{
							Error.ContainerName = FPaths::GetBaseFilename(Reader->GetContainerFile()->FilePath);
						}
					}
					Error.BlockIndex = CompressedBlock->Key.BlockIndex;
					Error.ExpectedHash = *CompressedBlock->SignatureHash;
					Error.ActualHash = BlockHash;
				}

				UE_LOGF(LogIoDispatcher, Warning, "Signature error detected in container '%ls' at block index '%d'", *Error.ContainerName, Error.BlockIndex);

				check(BackendContext);
				if (BackendContext->SignatureErrorDelegate.IsBound())
				{
					BackendContext->SignatureErrorDelegate.Broadcast(Error);
				}
			}
		}

		if (CompressedBlock->EncryptionKey.IsValid())
		{
			FAES::DecryptData(CompressedBuffer, CompressedBlock->RawSize, CompressedBlock->EncryptionKey);
		}
		uint8* UncompressedBuffer;
		if (CompressedBlock->CompressionMethod.IsNone())
		{
			UncompressedBuffer = CompressedBuffer;
		}
		else
		{
			if (CompressionContext->UncompressedBufferSize < CompressedBlock->UncompressedSize)
			{
				FMemory::Free(CompressionContext->UncompressedBuffer);
				CompressionContext->UncompressedBuffer = reinterpret_cast<uint8*>(FMemory::Malloc(CompressedBlock->UncompressedSize));
				CompressionContext->UncompressedBufferSize = CompressedBlock->UncompressedSize;
			}
			UncompressedBuffer = CompressionContext->UncompressedBuffer;

			bool bFailed = !FCompression::UncompressMemory(CompressedBlock->CompressionMethod, UncompressedBuffer, int32(CompressedBlock->UncompressedSize), CompressedBuffer, int32(CompressedBlock->CompressedSize));
			if (bFailed)
			{
				UE_LOGF(LogIoDispatcher, Warning, "Failed decompressing block");
				CompressedBlock->bFailed = true;
			}
		}

		for (FFileIoStoreBlockScatter& Scatter : CompressedBlock->ScatterList)
		{
			if (Scatter.Size)
			{
				check(Scatter.DstOffset + Scatter.Size <= Scatter.Request->GetBuffer().DataSize());
				check(Scatter.SrcOffset + Scatter.Size <= CompressedBlock->UncompressedSize);
				FMemory::Memcpy(Scatter.Request->GetBuffer().Data() + Scatter.DstOffset, UncompressedBuffer + Scatter.SrcOffset, Scatter.Size);
			}
		}
	}

	if (bIsAsync)
	{
		FScopeLock Lock(&DecompressedBlocksCritical);
		CompressedBlock->Next = FirstDecompressedBlock;
		FirstDecompressedBlock = CompressedBlock;
	}
}

void FFileIoStore::CompleteDispatcherRequest(FFileIoStoreResolvedRequest* ResolvedRequest)
{
	check(ResolvedRequest);
	check(ResolvedRequest->DispatcherRequest);

	ResolvedRequest->ContainerFile->RefCount.fetch_sub(1, std::memory_order_acq_rel);
	FIoRequestImpl* DispatcherRequest = ResolvedRequest->DispatcherRequest;
	ResolvedRequest->DispatcherRequest = nullptr;
	if (ResolvedRequest->bFailed)
	{
		DispatcherRequest->SetFailed();
		TRACE_IOSTORE_BACKEND_REQUEST_FAILED(DispatcherRequest);
	}
	else
	{
		TRACE_IOSTORE_BACKEND_REQUEST_COMPLETED(DispatcherRequest, ResolvedRequest->GetResolvedSize());
	}
	DispatcherRequest->BackendData = nullptr;
	if (!CompletedRequestsTail)
	{
		CompletedRequestsHead = CompletedRequestsTail = DispatcherRequest;
	}
	else
	{
		CompletedRequestsTail->NextRequest = DispatcherRequest;
		CompletedRequestsTail = DispatcherRequest;
	}
	CompletedRequestsTail->NextRequest = nullptr;
}

void FFileIoStore::FinalizeCompressedBlock(FFileIoStoreCompressedBlock* CompressedBlock)
{
	Stats.OnDecompressComplete(CompressedBlock); 

	if (CompressedBlock->RawBlocks.Num() > 1)
	{
		check(CompressedBlock->CompressedDataBuffer || CompressedBlock->bCancelled || CompressedBlock->bFailed);
		if (CompressedBlock->CompressedDataBuffer)
		{
			FMemory::Free(CompressedBlock->CompressedDataBuffer);
		}
	}
	else
	{
		FFileIoStoreReadRequest* RawBlock = CompressedBlock->RawBlocks[0];
		check(RawBlock->BufferRefCount > 0);
		if (--RawBlock->BufferRefCount == 0)
		{
			check(RawBlock->Buffer || RawBlock->bCancelled || RawBlock->bFailed);
			if (RawBlock->Buffer)
			{
				FreeBuffer(*RawBlock->Buffer);
				RawBlock->Buffer = nullptr;
			}
		}
	}
	check(CompressedBlock->CompressionContext || CompressedBlock->bCancelled || CompressedBlock->bFailed);
	if (CompressedBlock->CompressionContext)
	{
		FreeCompressionContext(CompressedBlock->CompressionContext);
	}
	for (int32 ScatterIndex = 0, ScatterCount = CompressedBlock->ScatterList.Num(); ScatterIndex < ScatterCount; ++ScatterIndex)
	{
		FFileIoStoreBlockScatter& Scatter = CompressedBlock->ScatterList[ScatterIndex];
		Stats.OnBytesScattered(Scatter.Size);
		Scatter.Request->bFailed |= CompressedBlock->bFailed;
		check(!CompressedBlock->bCancelled || !Scatter.Request->DispatcherRequest || Scatter.Request->DispatcherRequest->IsCancelled());
		check(Scatter.Request->UnfinishedReadsCount > 0);
		if (--Scatter.Request->UnfinishedReadsCount == 0)
		{
			if (!Scatter.Request->bCancelled)
			{
				CompleteDispatcherRequest(Scatter.Request);
			}
			RequestTracker.ReleaseIoRequestReferences(*Scatter.Request);
		}
	}
}

namespace FileIoStoreImpl
{
	static std::atomic<int32> ActiveScatterTasks{ 0 };

	bool HasActiveScatterTasks()
	{
		return ActiveScatterTasks.load(std::memory_order_relaxed) > 0;
	}

	bool IsSchedulerOversubscribed(UE::Tasks::ETaskPriority TaskPriority)
	{
		return LowLevelTasks::FScheduler::Get().IsOversubscriptionLimitReached(TaskPriority);
	}
}

FIoRequestImpl* FFileIoStore::GetCompletedIoRequests()
{
	LLM_SCOPE_BYNAME(TEXT("FileSystem/FileIoStore"));
	//TRACE_CPUPROFILER_EVENT_SCOPE(GetCompletedRequests);
	
	if (!bIsMultithreaded)
	{
		while (PlatformImpl->StartRequests(RequestQueue));
	}

	FFileIoStoreReadRequestList CompletedRequests;
	PlatformImpl->GetCompletedRequests(CompletedRequests);
	Stats.OnReadRequestsCompleted(CompletedRequests);
	for (auto It = CompletedRequests.Steal(); It; ++It)
	{
		FFileIoStoreReadRequest* CompletedRequest = *It;

		check(CompletedRequest->QueueStatus == FFileIoStoreReadRequest::QueueStatus_Started);
		CompletedRequest->QueueStatus = FFileIoStoreReadRequest::QueueStatus_Completed;
		int32 PreviousStartedReadRequestsCount = CompletedRequest->ContainerFilePartition->StartedReadRequestsCount.fetch_sub(1, std::memory_order_release);
		check(PreviousStartedReadRequestsCount >= 1);

		if (!CompletedRequest->ImmediateScatter.Request)
		{
			check(CompletedRequest->Buffer || CompletedRequest->bCancelled || CompletedRequest->bFailed);
			RequestTracker.RemoveRawBlock(CompletedRequest);
			
			//TRACE_CPUPROFILER_EVENT_SCOPE(ProcessCompletedBlock);
			for (FFileIoStoreCompressedBlock* CompressedBlock : CompletedRequest->CompressedBlocks)
			{
				CompressedBlock->bFailed |= CompletedRequest->bFailed;
				CompressedBlock->bCancelled |= CompletedRequest->bCancelled;
				if (CompressedBlock->RawBlocks.Num() > 1)
				{
					//TRACE_CPUPROFILER_EVENT_SCOPE(HandleComplexBlock);
					if (!(CompressedBlock->bCancelled | CompressedBlock->bFailed))
					{
						check(CompletedRequest->Buffer);
						if (!CompressedBlock->CompressedDataBuffer)
						{
							CompressedBlock->CompressedDataBuffer = reinterpret_cast<uint8*>(FMemory::Malloc(CompressedBlock->RawSize));
						}

						uint8* Src = CompletedRequest->Buffer->Memory;
						uint8* Dst = CompressedBlock->CompressedDataBuffer;
						uint64 CopySize = CompletedRequest->Size;
						int64 CompletedBlockOffsetInBuffer = int64(CompletedRequest->Offset) - int64(CompressedBlock->RawOffset);
						if (CompletedBlockOffsetInBuffer < 0)
						{
							Src -= CompletedBlockOffsetInBuffer;
							CopySize += CompletedBlockOffsetInBuffer;
						}
						else
						{
							Dst += CompletedBlockOffsetInBuffer;
						}
						uint64 CompressedBlockRawEndOffset = CompressedBlock->RawOffset + CompressedBlock->RawSize;
						uint64 CompletedBlockEndOffset = CompletedRequest->Offset + CompletedRequest->Size;
						if (CompletedBlockEndOffset > CompressedBlockRawEndOffset)
						{
							CopySize -= CompletedBlockEndOffset - CompressedBlockRawEndOffset;
						}
						FMemory::Memcpy(Dst, Src, CopySize);
					}
					check(CompletedRequest->BufferRefCount > 0);
					if (--CompletedRequest->BufferRefCount == 0)
					{
						if (CompletedRequest->Buffer)
						{
							FreeBuffer(*CompletedRequest->Buffer);
							CompletedRequest->Buffer = nullptr;
						}
					}
				}

				check(CompressedBlock->UnfinishedRawBlocksCount > 0);
				if (--CompressedBlock->UnfinishedRawBlocksCount == 0)
				{
					Stats.OnDecompressQueued(CompressedBlock);
					RequestTracker.RemoveCompressedBlock(CompressedBlock);
					if (!ReadyForDecompressionTail)
					{
						ReadyForDecompressionHead = ReadyForDecompressionTail = CompressedBlock;
					}
					else
					{
						ReadyForDecompressionTail->Next = CompressedBlock;
						ReadyForDecompressionTail = CompressedBlock;
					}
					CompressedBlock->Next = nullptr;
				}
			}
		}
		else
		{
			check(!CompletedRequest->Buffer);
			Stats.OnBytesScattered(CompletedRequest->ImmediateScatter.Size);
			FFileIoStoreResolvedRequest* CompletedResolvedRequest = CompletedRequest->ImmediateScatter.Request;
			CompletedResolvedRequest->bFailed |= CompletedRequest->bFailed;
			check(!CompletedRequest->bCancelled || !CompletedResolvedRequest->DispatcherRequest || CompletedResolvedRequest->DispatcherRequest->IsCancelled());
			check(CompletedResolvedRequest->UnfinishedReadsCount > 0);
			if (--CompletedResolvedRequest->UnfinishedReadsCount == 0)
			{
				if (!CompletedResolvedRequest->bCancelled)
				{
					CompleteDispatcherRequest(CompletedResolvedRequest);
				}
				RequestTracker.ReleaseIoRequestReferences(*CompletedResolvedRequest);
			}
		}
	}
	
	FFileIoStoreCompressedBlock* BlockToReap;
	{
		FScopeLock Lock(&DecompressedBlocksCritical);
		BlockToReap = FirstDecompressedBlock;
		FirstDecompressedBlock = nullptr;
	}

	while (BlockToReap)
	{
		FFileIoStoreCompressedBlock* Next = BlockToReap->Next;
		FinalizeCompressedBlock(BlockToReap);
		BlockToReap = Next;
	}

	// Cleanup finished decompression tasks
	UE::Tasks::FTask* DecompressionTask;
	while ((DecompressionTask = DecompressionTasks.Peek()) != nullptr && DecompressionTask->IsCompleted())
	{
		DecompressionTasks.Dequeue();
	}

	// Help with decompression to avoid deadlock on low-core count when all tasks threads are busy or waiting on IO requests.
	if (GIoDispatcherCanDecompressOnStarvation && !FileIoStoreImpl::HasActiveScatterTasks() && (DecompressionTask = DecompressionTasks.Peek()) != nullptr)
	{
		if (FileIoStoreImpl::IsSchedulerOversubscribed(DecompressionTask->GetPriority()))
		{
			// Try to execute it on the current thread if not already started, no-op if already started.
			DecompressionTask->TryRetractAndExecute();
			// In both case we can get rid of it right away since we know progress is being made.
			DecompressionTasks.Dequeue();
		}
	}

	FFileIoStoreCompressedBlock* BlockToDecompress = ReadyForDecompressionHead;
	while (BlockToDecompress)
	{
		FFileIoStoreCompressedBlock* Next = BlockToDecompress->Next;
		if (BlockToDecompress->bFailed | BlockToDecompress->bCancelled)
		{
			FinalizeCompressedBlock(BlockToDecompress);
			BlockToDecompress = Next;
			continue;
		}
		
		BlockToDecompress->CompressionContext = AllocCompressionContext();
		if (!BlockToDecompress->CompressionContext)
		{
			break;
		}

		for (const FFileIoStoreBlockScatter& Scatter : BlockToDecompress->ScatterList)
		{
			if (Scatter.Size)
			{
				FIoRequestImpl* DispatcherRequest = Scatter.Request->DispatcherRequest;
				check(DispatcherRequest);
				if (!DispatcherRequest->HasBuffer())
				{
					DispatcherRequest->CreateBuffer(Scatter.Request->ResolvedSize);
				}
			}
		}

		const UE::Tasks::ETaskPriority IoDispatcherTaskPriority =
			EnumHasAnyFlags(CPrio_IoDispatcherTaskPriority.Get(), ENamedThreads::BackgroundThreadPriority) ?
				UE::Tasks::ETaskPriority::BackgroundNormal :
				UE::Tasks::ETaskPriority::Normal;

		// Scatter block asynchronous when the block is compressed, encrypted or signed
		const bool bScatterAsync = bIsMultithreaded && GIoDispatcherForceSynchronousScatter == 0 &&
			(!BlockToDecompress->CompressionMethod.IsNone() ||
			 BlockToDecompress->EncryptionKey.IsValid() ||
			 BlockToDecompress->SignatureHash) && 
			 // If we're already oversubscribed, we might not receive any further event to wake us and 
			 // allow us to process our queue. In that case we simply run decompression locally.
			 !FileIoStoreImpl::IsSchedulerOversubscribed(IoDispatcherTaskPriority);

		if (bScatterAsync)
		{
			DecompressionTasks.Enqueue(
				UE::Tasks::Launch(
					TEXT("ScatterBlockDecompressionTask"),
					[this, BlockToDecompress]
					{
						FileIoStoreImpl::ActiveScatterTasks++;
						ScatterBlock(BlockToDecompress, true);
						FileIoStoreImpl::ActiveScatterTasks--;

						// Important that the notification goes after the decrement of the active scatter tasks
						// otherwise we could end up missing an event and deadlock.
						BackendContext->WakeUpDispatcherThreadDelegate.Execute();
					},
					IoDispatcherTaskPriority
				)
			);
		}
		else
		{
			ScatterBlock(BlockToDecompress, false);
			FinalizeCompressedBlock(BlockToDecompress);
		}
		BlockToDecompress = Next;
	}
	ReadyForDecompressionHead = BlockToDecompress;
	if (!ReadyForDecompressionHead)
	{
		ReadyForDecompressionTail = nullptr;
	}

	IoStoreReadersToUnload.TryUnload();

	FIoRequestImpl* Result = CompletedRequestsHead;
	CompletedRequestsHead = CompletedRequestsTail = nullptr;
	return Result;
}

TIoStatusOr<FIoMappedRegion> FFileIoStore::OpenMapped(const FIoChunkId& ChunkId, const FIoReadOptions& Options)
{
	if (!FPlatformProperties::SupportsMemoryMappedFiles())
	{
		return FIoStatus(EIoErrorCode::Unknown, TEXT("Platform does not support memory mapped files"));
	}

	if (Options.GetTargetVa() != nullptr)
	{
		return FIoStatus(EIoErrorCode::InvalidParameter, TEXT("Invalid read options"));
	}

	FReadScopeLock _(IoStoreReadersLock);
	for (TUniquePtr<FFileIoStoreReader>& Reader : IoStoreReaders)
	{
		if (const FIoOffsetAndLength* OffsetAndLength = Reader->Resolve(ChunkId))
		{
			uint64 ResolvedOffset = OffsetAndLength->GetOffset();
			uint64 ResolvedSize = FMath::Min(Options.GetSize(), OffsetAndLength->GetLength());
			
			const FFileIoStoreContainerFile* ContainerFile = Reader->GetContainerFile();
			
			int32 BlockIndex = int32(ResolvedOffset / ContainerFile->CompressionBlockSize);
			const FIoStoreTocCompressedBlockEntry& CompressionBlockEntry = ContainerFile->CompressionBlocks[BlockIndex];
			const int64 BlockOffset = (int64)CompressionBlockEntry.GetOffset();
			// This is not required as mapped file region implementation ensure we map an aligned
			// pointer/size, handling arbitrary offset/size.
			// check(BlockOffset > 0 && IsAligned(BlockOffset, FPlatformProperties::GetMemoryMappingAlignment()));
			TUniquePtr<IMappedFileHandle> MappedFileHandle = Reader->GetMappedContainerFileHandle(BlockOffset);
			const int64 MappingOffset = BlockOffset + Options.GetOffset();
			IMappedFileRegion* MappedFileRegion = MappedFileHandle->MapRegion(MappingOffset, ResolvedSize);
			if (MappedFileRegion != nullptr)
			{
				// MappedPtr could be inside of the page, depending on the offset. This
				// is actually fine, unless client code cares about MappedPtr alignment, which is
				// public API.
				// check(IsAligned(MappedFileRegion->GetMappedPtr(), FPlatformMemory::GetStats().PageSize));
				return FIoMappedRegion{ MappedFileHandle.Release(), MappedFileRegion };
			}
			else
			{
				const FString& FilePath = Reader->GetContainerFile()->FilePath;
				UE_LOG(LogIoDispatcher, Error, TEXT("MapRegion failed on: %s (0x%016" INT64_x_FMT ", size: %" UINT64_FMT ")"), *FilePath, MappingOffset, ResolvedSize);
				return FIoStatus(EIoErrorCode::ReadError);
			}
		}
	}

	// We didn't find any entry for the ChunkId.
	return FIoStatus(EIoErrorCode::NotFound);
}

const TCHAR* FFileIoStore::GetName() const
{
	return TEXT("PakFile");
}

void FFileIoStore::ReopenAllFileHandles()
{
	UE_CLOGF(RequestTracker.GetLiveReadRequestsCount(), LogIoDispatcher, Warning, "Calling ReopenAllFileHandles with read requests in flight");
	FWriteScopeLock _(IoStoreReadersLock);
	for (const TUniquePtr<FFileIoStoreReader>& Reader : IoStoreReaders)
	{
		Reader->ReopenAllFileHandles();
	}
}

void FFileIoStore::OnNewPendingRequestsAdded()
{
	if (bIsMultithreaded)
	{
		PlatformImpl->ServiceNotify();
	}
}

void FFileIoStore::ReadBlocks(FFileIoStoreResolvedRequest& ResolvedRequest)
{
	/*TStringBuilder<256> ScopeName;
	ScopeName.Appendf(TEXT("ReadBlock %d"), BlockIndex);
	TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*ScopeName);*/

	FFileIoStoreContainerFile* ContainerFile = ResolvedRequest.GetContainerFile();
	const uint64 CompressionBlockSize = ContainerFile->CompressionBlockSize;
	const uint64 RequestEndOffset = ResolvedRequest.ResolvedOffset + ResolvedRequest.ResolvedSize;
	int32 RequestBeginBlockIndex = int32(ResolvedRequest.ResolvedOffset / CompressionBlockSize);
	int32 RequestEndBlockIndex = int32((RequestEndOffset - 1) / CompressionBlockSize);

	FFileIoStoreReadRequestList NewBlocks;

	uint64 RequestStartOffsetInBlock = ResolvedRequest.ResolvedOffset - RequestBeginBlockIndex * CompressionBlockSize;
	uint64 RequestRemainingBytes = ResolvedRequest.ResolvedSize;
	uint64 OffsetInRequest = 0;
	for (int32 CompressedBlockIndex = RequestBeginBlockIndex; CompressedBlockIndex <= RequestEndBlockIndex; ++CompressedBlockIndex)
	{
		FFileIoStoreBlockKey CompressedBlockKey;
		CompressedBlockKey.FileIndex = ContainerFile->ContainerInstanceId;
		CompressedBlockKey.BlockIndex = CompressedBlockIndex;
		bool bCompressedBlockWasAdded;
		FFileIoStoreCompressedBlock* CompressedBlock = RequestTracker.FindOrAddCompressedBlock(CompressedBlockKey, bCompressedBlockWasAdded);
		check(CompressedBlock);
		check(!CompressedBlock->bCancelled);
		if (bCompressedBlockWasAdded)
		{
			CompressedBlock->EncryptionKey = ContainerFile->EncryptionKey;
			const FIoStoreTocCompressedBlockEntry& CompressionBlockEntry = ContainerFile->CompressionBlocks[CompressedBlockIndex];
			CompressedBlock->UncompressedSize = CompressionBlockEntry.GetUncompressedSize();
			CompressedBlock->CompressedSize = CompressionBlockEntry.GetCompressedSize();
			CompressedBlock->CompressionMethod = ContainerFile->CompressionMethods[CompressionBlockEntry.GetCompressionMethodIndex()];
			if (EnumHasAnyFlags(ContainerFile->ContainerFlags, EIoContainerFlags::Signed))
			{
				check(!ContainerFile->BlockSignatureTable.IsEmpty());
				CompressedBlock->BlockSignatureTable = ContainerFile->BlockSignatureTable;
				CompressedBlock->SignatureHash = &ContainerFile->BlockSignatureTable[CompressedBlockIndex];
			}
			CompressedBlock->RawSize = Align(CompressionBlockEntry.GetCompressedSize(), FAES::AESBlockSize); // The raw blocks size is always aligned to AES blocks size;

			int32 PartitionIndex = int32(CompressionBlockEntry.GetOffset() / ContainerFile->PartitionSize);
			FFileIoStoreContainerFilePartition& Partition = ContainerFile->Partitions[PartitionIndex];
			uint64 PartitionRawOffset = CompressionBlockEntry.GetOffset() % ContainerFile->PartitionSize;
			CompressedBlock->RawOffset = PartitionRawOffset;
			const uint32 RawBeginBlockIndex = uint32(PartitionRawOffset / ReadBufferSize);
			const uint32 RawEndBlockIndex = uint32((PartitionRawOffset + CompressedBlock->RawSize - 1) / ReadBufferSize);
			const uint32 RawBlockCount = RawEndBlockIndex - RawBeginBlockIndex + 1;
			check(RawBlockCount > 0);
			for (uint32 RawBlockIndex = RawBeginBlockIndex; RawBlockIndex <= RawEndBlockIndex; ++RawBlockIndex)
			{
				FFileIoStoreBlockKey RawBlockKey;
				RawBlockKey.BlockIndex = RawBlockIndex;
				RawBlockKey.FileIndex = Partition.ContainerFileIndex;

				bool bRawBlockWasAdded;
				FFileIoStoreReadRequest* RawBlock = RequestTracker.FindOrAddRawBlock(RawBlockKey, bRawBlockWasAdded);
				check(RawBlock);
				check(!RawBlock->bCancelled);
				if (bRawBlockWasAdded)
				{
					RawBlock->Priority = ResolvedRequest.GetPriority();
					RawBlock->ContainerFilePartition = &Partition;
					RawBlock->Offset = RawBlockIndex * ReadBufferSize;
					uint64 ReadSize = FMath::Min(Partition.FileSize, RawBlock->Offset + ReadBufferSize) - RawBlock->Offset;
					RawBlock->Size = ReadSize;
					NewBlocks.Add(RawBlock);
				}
				RawBlock->BytesUsed += 
					uint32(FMath::Min(CompressedBlock->RawOffset + CompressedBlock->RawSize, RawBlock->Offset + RawBlock->Size) -
						   FMath::Max(CompressedBlock->RawOffset, RawBlock->Offset));
				CompressedBlock->RawBlocks.Add(RawBlock);
				++CompressedBlock->UnfinishedRawBlocksCount;
				++CompressedBlock->RefCount;
				RawBlock->CompressedBlocks.Add(CompressedBlock);
				++RawBlock->BufferRefCount;
			}
		}
		check(CompressedBlock->UncompressedSize > RequestStartOffsetInBlock);
		uint64 RequestSizeInBlock = FMath::Min<uint64>(CompressedBlock->UncompressedSize - RequestStartOffsetInBlock, RequestRemainingBytes);
		check(OffsetInRequest + RequestSizeInBlock <= ResolvedRequest.ResolvedSize);
		check(RequestStartOffsetInBlock + RequestSizeInBlock <= CompressedBlock->UncompressedSize);

		FFileIoStoreBlockScatter& Scatter = CompressedBlock->ScatterList.AddDefaulted_GetRef();
		Scatter.Request = &ResolvedRequest;
		Scatter.DstOffset = OffsetInRequest;
		Scatter.SrcOffset = RequestStartOffsetInBlock;
		Scatter.Size = RequestSizeInBlock;

		RequestRemainingBytes -= RequestSizeInBlock;
		OffsetInRequest += RequestSizeInBlock;
		RequestStartOffsetInBlock = 0;

		RequestTracker.AddReadRequestsToResolvedRequest(CompressedBlock, ResolvedRequest);
	}

	if (!NewBlocks.IsEmpty())
	{
		Stats.OnReadRequestsQueued(NewBlocks);
		RequestQueue.Push(NewBlocks);
		OnNewPendingRequestsAdded();
	}
}

void FFileIoStore::FreeBuffer(FFileIoStoreBuffer& Buffer)
{
	BufferAllocator.FreeBuffer(&Buffer);
	PlatformImpl->ServiceNotify();
}

FFileIoStoreCompressionContext* FFileIoStore::AllocCompressionContext()
{
	FFileIoStoreCompressionContext* Result = FirstFreeCompressionContext;
	if (Result)
	{
		FirstFreeCompressionContext = FirstFreeCompressionContext->Next;
	}
	return Result;
}

void FFileIoStore::FreeCompressionContext(FFileIoStoreCompressionContext* CompressionContext)
{
	CompressionContext->Next = FirstFreeCompressionContext;
	FirstFreeCompressionContext = CompressionContext;
}

bool FFileIoStore::Init()
{
	return true;
}

void FFileIoStore::Stop()
{
	bStopRequested = true;
	PlatformImpl->ServiceNotify();
}

uint32 FFileIoStore::Run()
{
	while (!bStopRequested)
	{
		if (!PlatformImpl->StartRequests(RequestQueue))
		{
			PlatformImpl->ServiceWait();
		}
	}
	return 0;
}

TSharedRef<FFileIoStore> CreateIoDispatcherFileBackend()
{
	bool bCheckForPlatformImplementation = true;
	if (!FGenericPlatformProcess::SupportsMultithreading())
	{
		bCheckForPlatformImplementation = false;
	}
#if !UE_BUILD_SHIPPING
	if (FParse::Param(FCommandLine::Get(), TEXT("forcegenericio")))
	{
		bCheckForPlatformImplementation = false;
	}
#endif

	if (bCheckForPlatformImplementation)
	{
		if (FModuleManager::Get().ModuleExists(ANSI_TO_TCHAR(PLATFORM_IODISPATCHER_MODULE)))
		{
			IPlatformFileIoStoreModule* PlatformModule = FModuleManager::LoadModulePtr<IPlatformFileIoStoreModule>(ANSI_TO_TCHAR(PLATFORM_IODISPATCHER_MODULE));
			if (PlatformModule)
			{
				TUniquePtr<IPlatformFileIoStore> PlatformImpl = PlatformModule->CreatePlatformFileIoStore();
				if (PlatformImpl.IsValid())
				{
					return MakeShared<FFileIoStore>(MoveTemp(PlatformImpl));
				}
			}
		}
#if PLATFORM_IMPLEMENTS_IO
		{
			TUniquePtr<IPlatformFileIoStore> PlatformImpl = CreatePlatformFileIoStore();
			if (PlatformImpl.IsValid())
			{
				return MakeShared<FFileIoStore>(MoveTemp(PlatformImpl));
			}
		}
#endif
	}
	return MakeShared<FFileIoStore>(MakeUnique<FGenericFileIoStoreImpl>());
}

uint32 FFileIoStore::GetThreadId() const
{
	return Thread ? Thread->GetThreadID() : 0; 
}

#if UE_FILEIOSTORE_STATS_ENABLED

FFileIoStoreStats::FFileIoStoreStats()
{
}

FFileIoStoreStats::~FFileIoStoreStats()
{
}

void FFileIoStoreStats::OnReadRequestsQueued(const FFileIoStoreReadRequestList& Requests)
{
	uint64 TotalBytes = 0;
	int32 NumReads = 0;
	for (const FFileIoStoreReadRequest* Request : Requests)
	{
		++NumReads;
		TotalBytes += Request->Size;
	}

	Stats.OnReadRequestsQueued(TotalBytes, NumReads);
}

void FFileIoStoreStats::OnFilesystemReadStarted(const FFileIoStoreReadRequest* Request)
{
	Stats.OnFilesystemReadStarted(uint64(Request->ContainerFilePartition), Request->Offset, Request->Size);
}

void FFileIoStoreStats::OnFilesystemReadsStarted(const FFileIoStoreReadRequestList& Requests)
{
	for (const FFileIoStoreReadRequest* Request : Requests)
	{
		Stats.OnFilesystemReadStarted(uint64(Request->ContainerFilePartition), Request->Offset, Request->Size);
	}
}

void FFileIoStoreStats::OnFilesystemReadCompleted(const FFileIoStoreReadRequest* CompletedRequest)
{
	Stats.OnFilesystemReadCompleted(uint64(CompletedRequest->ContainerFilePartition), CompletedRequest->Offset, CompletedRequest->Size);
}

void FFileIoStoreStats::OnFilesystemReadsCompleted(const FFileIoStoreReadRequestList& CompletedRequests)
{
	for (const FFileIoStoreReadRequest* Request : CompletedRequests)
	{
		Stats.OnFilesystemReadCompleted(uint64(Request->ContainerFilePartition), Request->Offset, Request->Size);
	}
}

void FFileIoStoreStats::OnReadRequestsCompleted(const FFileIoStoreReadRequestList& CompletedRequests)
{
	int64 TotalBytes = 0;
	int32 NumReads = 0;
	for (const FFileIoStoreReadRequest* Request : CompletedRequests)
	{
		++NumReads;
		TotalBytes += Request->Size;
	}
	Stats.OnReadRequestsCompleted(TotalBytes, NumReads);
}

void FFileIoStoreStats::OnDecompressQueued(const FFileIoStoreCompressedBlock* CompressedBlock)
{
	Stats.OnDecompressQueued(CompressedBlock->CompressedSize, CompressedBlock->UncompressedSize);
}

void FFileIoStoreStats::OnDecompressComplete(const FFileIoStoreCompressedBlock* CompressedBlock)
{
	Stats.OnDecompressComplete(CompressedBlock->CompressedSize, CompressedBlock->UncompressedSize);
}

void FFileIoStoreStats::OnBytesScattered(int64 NumBytes)
{
	Stats.OnBytesScattered(NumBytes);
}

void FFileIoStoreStats::OnBlockCacheStore(uint64 NumBytes)
{
	Stats.OnBlockCacheStore(NumBytes);
}

void FFileIoStoreStats::OnBlockCacheHit(uint64 NumBytes)
{
	Stats.OnBlockCacheHit(NumBytes);
}

void FFileIoStoreStats::OnBlockCacheMiss(uint64 NumBytes)
{
	Stats.OnBlockCacheMiss(NumBytes);
}

void FFileIoStoreStats::OnTocMounted(uint64 AllocatedSize)
{
	Stats.OnTocMounted(AllocatedSize);
}

void FFileIoStoreStats::OnTocUnmounted(uint64 AllocatedSize)
{
	Stats.OnTocUnmounted(AllocatedSize);
}

void FFileIoStoreStats::OnBufferReleased()
{
	Stats.OnBufferReleased();
}

void FFileIoStoreStats::OnBufferAllocated()
{
	Stats.OnBufferAllocated();
}

#endif
