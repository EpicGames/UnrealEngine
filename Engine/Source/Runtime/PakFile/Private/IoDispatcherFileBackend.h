// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FileIoDispatcherBackend.h"

#include "Async/Mutex.h"
#include "Async/UniqueLock.h"
#include "IoDispatcherFileBackendTypes.h"
#include "IO/IoDispatcher.h"
#include "IO/IoStore.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/SpscQueue.h"
#include "Stats/Stats.h"
#include "Tasks/Task.h"
#include "HAL/Runnable.h"
#include "Misc/AES.h"
#include "GenericPlatform/GenericPlatformFile.h"

#include <atomic>

class IMappedFileHandle;

struct FFileIoStoreCompressionContext
{
	FFileIoStoreCompressionContext* Next = nullptr;
	uint64 UncompressedBufferSize = 0;
	uint8* UncompressedBuffer = nullptr;
};

class FFileIoStoreReader
{
public:
	FFileIoStoreReader(IPlatformFileIoStore& InPlatformImpl, FFileIoStoreStats& InStats);
	~FFileIoStoreReader();
	FIoStatus Initialize(const TCHAR* InTocFilePath, int32 Order);
	uint32 GetContainerInstanceId() const
	{
		return ContainerFile.ContainerInstanceId;
	}
	FIoStatus Close();
	bool DoesChunkExist(const FIoChunkId& ChunkId) const;
	TIoStatusOr<uint64> GetSizeForChunk(const FIoChunkId& ChunkId) const;
	const FIoOffsetAndLength* Resolve(const FIoChunkId& ChunkId) const;
	FFileIoStoreContainerFile* GetContainerFile() { return &ContainerFile; }
	const FFileIoStoreContainerFile* GetContainerFile() const { return &ContainerFile; }
	TUniquePtr<IMappedFileHandle> GetMappedContainerFileHandle(uint64 TocOffset);
	const FIoContainerId& GetContainerId() const { return ContainerId; }
	int32 GetOrder() const { return Order; }
	bool IsEncrypted() const { return EnumHasAnyFlags(ContainerFile.ContainerFlags, EIoContainerFlags::Encrypted); }
	bool IsSigned() const { return EnumHasAnyFlags(ContainerFile.ContainerFlags, EIoContainerFlags::Signed); }
	const FGuid& GetEncryptionKeyGuid() const { return ContainerFile.EncryptionKeyGuid; }
	void SetEncryptionKey(const FAES::FAESKey& Key) { ContainerFile.EncryptionKey = Key; }
	const FAES::FAESKey& GetEncryptionKey() const { return ContainerFile.EncryptionKey; }
	TIoStatusOr<FIoContainerHeader> ReadContainerHeader(bool bReadSoftRefs) const;
	void ReopenAllFileHandles();

private:
	const FIoOffsetAndLength* FindChunkInternal(const FIoChunkId& ChunkId) const;
	uint64 GetTocAllocatedSize() const;
	
	IPlatformFileIoStore& PlatformImpl;
	FFileIoStoreStats& Stats;

	struct FPerfectHashMap
	{
		TConstArrayView<int32> TocChunkHashSeeds;
		TConstArrayView<FIoChunkId> TocChunkIds;
		TConstArrayView<FIoOffsetAndLength> TocOffsetAndLengths;
	};
	FPerfectHashMap PerfectHashMap;
	TMap<FIoChunkId, FIoOffsetAndLength> TocImperfectHashMapFallback;
	FFileIoStoreContainerFile ContainerFile;
	FIoContainerId ContainerId;
	// Owns the data for ArrayViews in ContainerFile and in PerfectHash map
	FIoStoreTocResourceStorage DataContainer;
	int32 Order;
	bool bClosed = false;
	bool bHasPerfectHashMap = false;

	static TAtomic<uint32> GlobalPartitionIndex;
	static TAtomic<uint32> GlobalContainerInstanceId;
};

class FFileIoStoreRequestTracker
{
public:
	FFileIoStoreRequestTracker(FFileIoStoreRequestAllocator& RequestAllocator, FFileIoStoreRequestQueue& RequestQueue);
	~FFileIoStoreRequestTracker();

	FFileIoStoreCompressedBlock* FindOrAddCompressedBlock(FFileIoStoreBlockKey Key, bool& bOutWasAdded);
	void RemoveCompressedBlock(const FFileIoStoreCompressedBlock* CompressedBlock, bool bRemoveFromCancel = false);
	FFileIoStoreReadRequest* FindOrAddRawBlock(FFileIoStoreBlockKey Key, bool& bOutWasAdded);
	void RemoveRawBlock(const FFileIoStoreReadRequest* RawBlock, bool bRemoveFromCancel = false);
	void AddReadRequestsToResolvedRequest(FFileIoStoreCompressedBlock* CompressedBlock, FFileIoStoreResolvedRequest& ResolvedRequest);
	void AddReadRequestsToResolvedRequest(const FFileIoStoreReadRequestList& Requests, FFileIoStoreResolvedRequest& ResolvedRequest);
	bool CancelIoRequest(FFileIoStoreResolvedRequest& ResolvedRequest);
	void UpdatePriorityForIoRequest(FFileIoStoreResolvedRequest& ResolvedRequest);
	void ReleaseIoRequestReferences(FFileIoStoreResolvedRequest& ResolvedRequest);
	int64 GetLiveReadRequestsCount() const;

private:
	FFileIoStoreRequestAllocator& RequestAllocator;
	FFileIoStoreRequestQueue& RequestQueue;
	TMap<FFileIoStoreBlockKey, FFileIoStoreCompressedBlock*> CompressedBlocksMap;
	TMap<FFileIoStoreBlockKey, FFileIoStoreReadRequest*> RawBlocksMap;
};

class FFileIoStoreReadersToUnload
{
public:
	FFileIoStoreReadersToUnload() = default;
	~FFileIoStoreReadersToUnload();
	void Add(TUniquePtr<FFileIoStoreReader>&& Reader);
	void TryUnload();

private:
	TArray<TUniquePtr<FFileIoStoreReader>> Readers;
	UE::FMutex Mutex;
};

class FFileIoStore final
	: public FRunnable
	, public UE::IoStore::IFileIoDispatcherBackend
{
public:
	FFileIoStore(TUniquePtr<IPlatformFileIoStore>&& PlatformImpl);
	~FFileIoStore();
	void Initialize(TSharedRef<const FIoDispatcherBackendContext> Context) override;
	void Shutdown() override;
	virtual TIoStatusOr<FIoContainerHeader> Mount(
		const TCHAR* InTocPath, 
		int32 Order, 
		const FGuid& EncryptionKeyGuid, 
		const FAES::FAESKey& EncryptionKey, 
		UE::IoStore::ETocMountOptions Options) override;
	virtual bool Unmount(const TCHAR* InTocPath) override;
	void ResolveIoRequests(FIoRequestList Requests, FIoRequestList& OutUnresolved) override;
	void CancelIoRequest(FIoRequestImpl* Request) override;
	void UpdatePriorityForIoRequest(FIoRequestImpl* Request) override;
	bool DoesChunkExist(const FIoChunkId& ChunkId) const override;
	TIoStatusOr<uint64> GetSizeForChunk(const FIoChunkId& ChunkId) const;
	FIoRequestImpl* GetCompletedIoRequests() override;
	TIoStatusOr<FIoMappedRegion> OpenMapped(const FIoChunkId& ChunkId, const FIoReadOptions& Options) override;
	const TCHAR* GetName() const override;
	virtual void ReopenAllFileHandles() override;

	virtual bool Init() override;
	virtual uint32 Run() override;
	virtual void Stop() override;

	uint32 GetThreadId() const;

private:

	bool Resolve(FIoRequestImpl* Request);
	void OnNewPendingRequestsAdded();
	void ReadBlocks(FFileIoStoreResolvedRequest& ResolvedRequest);
	void FreeBuffer(FFileIoStoreBuffer& Buffer);
	FFileIoStoreCompressionContext* AllocCompressionContext();
	void FreeCompressionContext(FFileIoStoreCompressionContext* CompressionContext);
	void ScatterBlock(FFileIoStoreCompressedBlock* CompressedBlock, bool bIsAsync);
	void CompleteDispatcherRequest(FFileIoStoreResolvedRequest* ResolvedRequest);
	void FinalizeCompressedBlock(FFileIoStoreCompressedBlock* CompressedBlock);
	void StopThread();

	uint64 ReadBufferSize = 0;
	TSharedPtr<const FIoDispatcherBackendContext> BackendContext;
	FFileIoStoreStats Stats;
	FFileIoStoreBlockCache BlockCache;
	FFileIoStoreBufferAllocator BufferAllocator;
	FFileIoStoreRequestAllocator RequestAllocator;
	FFileIoStoreRequestQueue RequestQueue;
	FFileIoStoreRequestTracker RequestTracker;
	TUniquePtr<IPlatformFileIoStore> PlatformImpl;
	FFileIoStoreReadersToUnload IoStoreReadersToUnload;
	FRunnableThread* Thread = nullptr;
	bool bIsMultithreaded;
	TAtomic<bool> bStopRequested{ false };
	mutable FRWLock IoStoreReadersLock;
	TArray<TUniquePtr<FFileIoStoreReader>> IoStoreReaders;
	TArray<TUniquePtr<FFileIoStoreCompressionContext>> CompressionContexts;
	TSpscQueue<UE::Tasks::FTask> DecompressionTasks;
	FFileIoStoreCompressionContext* FirstFreeCompressionContext = nullptr;
	FFileIoStoreCompressedBlock* ReadyForDecompressionHead = nullptr;
	FFileIoStoreCompressedBlock* ReadyForDecompressionTail = nullptr;
	FCriticalSection DecompressedBlocksCritical;
	FFileIoStoreCompressedBlock* FirstDecompressedBlock = nullptr;
	FIoRequestImpl* CompletedRequestsHead = nullptr;
	FIoRequestImpl* CompletedRequestsTail = nullptr;
	FDelegateHandle OversubscriptionLimitReached;
};

TSharedRef<FFileIoStore> CreateIoDispatcherFileBackend();
