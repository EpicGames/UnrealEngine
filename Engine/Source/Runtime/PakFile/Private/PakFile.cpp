// Copyright Epic Games, Inc. All Rights Reserved.

#include "PakFile.h"
#include "Async/MappedFileHandle.h"
#include "HAL/FileManager.h"
#include "HAL/FileManagerGeneric.h"
#include "HAL/LowLevelMemTracker.h"
#include "IO/IoContainerHeader.h"
#include "IPlatformFilePak.h"
#include "Math/GuardedInt.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/EncryptionKeyManager.h"
#include "Misc/Fnv.h"
#include "Misc/Parse.h"
#include "Misc/ScopeExit.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "SignedArchiveReader.h"

#include "PakFile.inl"

LLM_DEFINE_TAG(PakSharedReaders);

int32	GetPakchunkIndexFromPakFile(const FString&);
bool	ShouldCheckPak();
void	DecryptData(uint8*, uint64, FGuid);

#if ENABLE_PAKFILE_RUNTIME_PRUNING
bool FPakFile::bSomePakNeedsPruning = false;
#endif

/*** This is a copy of FFnv::MemFnv64 from before the bugfix for swapped Offset
 * and Prime. It is used to decode legacy paks that have hashes created from
 * the prebugfix version of the function */
static uint64 LegacyMemFnv64(const void* InData, int32 Length, uint64 InOffset)
{
	// constants from above reference
	static const uint64 Offset = 0x00000100000001b3;
	static const uint64 Prime = 0xcbf29ce484222325;

	const uint8* __restrict Data = (uint8*)InData;

	uint64 Fnv = Offset + InOffset; // this is not strictly correct as the offset should be prime and InOffset could be arbitrary
	for (; Length; --Length)
	{
		Fnv ^= *Data++;
		Fnv *= Prime;
	}

	return Fnv;
}



struct FPakFile::FIndexSettings
{
	FIndexSettings()
	{
		bKeepFullDirectory = true;
		bValidatePruning = false;
		bDelayPruning = false;
		bWritePathHashIndex = true;
		bWriteFullDirectoryIndex = true;

		// Paks are mounted before config files are read, so the licensee needs to hardcode all settings used for runtime index loading rather than specifying them in ini
		if (FPakPlatformFile::GetPakSetIndexSettingsDelegate().IsBound())
		{
			FPakPlatformFile::GetPakSetIndexSettingsDelegate().Execute(bKeepFullDirectory, bValidatePruning, bDelayPruning);
		}

		// Settings not used at runtime can be read from ini
#if !UE_BUILD_SHIPPING
		GConfig->GetBool(TEXT("Pak"), TEXT("WritePathHashIndex"), bWritePathHashIndex, GEngineIni);
		GConfig->GetBool(TEXT("Pak"), TEXT("WriteFullDirectoryIndex"), bWriteFullDirectoryIndex, GEngineIni);
#endif

#if IS_PROGRAM || WITH_EDITOR
		// Directory pruning is not enabled in the editor or in development programs because there is no need to save the memory in those environments and some development features require not pruning
		bKeepFullDirectory = true;
#else
		bKeepFullDirectory = bKeepFullDirectory || !FPlatformProperties::RequiresCookedData();
#endif
#if !UE_BUILD_SHIPPING
		const TCHAR* CommandLine = FCommandLine::Get();
		FParse::Bool(CommandLine, TEXT("ForcePakKeepFullDirectory="), bKeepFullDirectory);
#if ENABLE_PAKFILE_RUNTIME_PRUNING_VALIDATE
		FParse::Bool(CommandLine, TEXT("ForcePakValidatePruning="), bValidatePruning);
		FParse::Bool(CommandLine, TEXT("ForcePakDelayPruning="), bDelayPruning);
#endif
		FParse::Bool(CommandLine, TEXT("ForcePakWritePathHashIndex="), bWritePathHashIndex);
		FParse::Bool(CommandLine, TEXT("ForcePakWriteFullDirectoryIndex="), bWriteFullDirectoryIndex);
#endif
	}

	bool bKeepFullDirectory;
	bool bValidatePruning;
	bool bDelayPruning;
	bool bWritePathHashIndex;
	bool bWriteFullDirectoryIndex;
};



#if IS_PROGRAM
FPakFile::FPakFile(const TCHAR* Filename, bool bIsSigned)
	: PakFilename(Filename)
	, PakFilenameName(Filename)
	, PathHashSeed(0)
	, NumEntries(0)
	, CachedTotalSize(0)
	, bSigned(bIsSigned)
	, bIsValid(false)
	, bHasPathHashIndex(false)
	, bHasFullDirectoryIndex(false)
#if ENABLE_PAKFILE_RUNTIME_PRUNING
	, bWillPruneDirectoryIndex(false)
	, bNeedsLegacyPruning(false)
#endif
#if ENABLE_PAKFILE_USE_DIRECTORY_TREE
	, bUseDirectoryTree(false)
#endif
	, PakchunkIndex(GetPakchunkIndexFromPakFile(Filename))
 	, MappedFileHandle(nullptr)
	, CacheType(FPakFile::ECacheType::Shared)
	, CacheIndex(-1)
	, UnderlyingCacheTrimDisabled(false)
	, bIsMounted(false)
{
	FSharedPakReader Reader = GetSharedReader(NULL);
	if (Reader)
	{
		Timestamp = IFileManager::Get().GetTimeStamp(Filename);
		Initialize(Reader.GetArchive());
	}
}
#endif

FPakFile::FPakFile(IPlatformFile* LowerLevel, const TCHAR* Filename, bool bIsSigned, bool bLoadIndex)
	: PakFilename(Filename)
	, PakFilenameName(Filename)
	, PathHashSeed(0)
	, NumEntries(0)
	, CachedTotalSize(0)
	, bSigned(bIsSigned)
	, bIsValid(false)
	, bHasPathHashIndex(false)
	, bHasFullDirectoryIndex(false)
#if ENABLE_PAKFILE_RUNTIME_PRUNING
	, bWillPruneDirectoryIndex(false)
	, bNeedsLegacyPruning(false)
#endif
#if ENABLE_PAKFILE_USE_DIRECTORY_TREE
	, bUseDirectoryTree(false)
#endif
	, PakchunkIndex(GetPakchunkIndexFromPakFile(Filename))
	, MappedFileHandle(nullptr)
	, CacheType(FPakFile::ECacheType::Shared)
	, CacheIndex(-1)
	, UnderlyingCacheTrimDisabled(false)
	, bIsMounted(false)
{
	FSharedPakReader Reader = GetSharedReader(LowerLevel);
	if (Reader)
	{
		Timestamp = LowerLevel->GetTimeStamp(Filename);
		Initialize(Reader.GetArchive(), bLoadIndex);
	}
}

#if WITH_EDITOR
FPakFile::FPakFile(FArchive* Archive)
	: PathHashSeed(0)
	, NumEntries(0)
	, bSigned(false)
	, bIsValid(false)
	, bHasPathHashIndex(false)
	, bHasFullDirectoryIndex(false)
#if ENABLE_PAKFILE_RUNTIME_PRUNING
	, bWillPruneDirectoryIndex(false)
	, bNeedsLegacyPruning(false)
#endif
#if ENABLE_PAKFILE_USE_DIRECTORY_TREE
	, bUseDirectoryTree(false)
#endif
	, PakchunkIndex(INDEX_NONE)
	, MappedFileHandle(nullptr)
	, CacheType(FPakFile::ECacheType::Shared)
	, CacheIndex(-1)
	, UnderlyingCacheTrimDisabled(false)
	, bIsMounted(false)
{
	Initialize(*Archive);
}
#endif

FPakFile::~FPakFile()
{
}

bool FPakFile::PassedSignatureChecks() const
{
	return Decryptor.IsValid() && Decryptor->IsValid();
}

FArchive* FPakFile::CreatePakReader(IPlatformFile* LowerLevel, const TCHAR* Filename)
{
	auto MakeArchive = [&]() -> FArchive* { 
		if (LowerLevel)
		{
			if( IFileHandle* Handle = LowerLevel->OpenRead(Filename) )
			{
				return new FArchiveFileReaderGeneric(Handle, Filename, Handle->Size());
			}
			else
			{
				return nullptr;
			}
		}
		else
		{
			return IFileManager::Get().CreateFileReader(Filename); 
		}
	};

	bool bNeedsDecryptor = false;
	if (FPlatformProperties::RequiresCookedData())
	{
		bool bShouldCheckSignature = bSigned || FParse::Param(FCommandLine::Get(), TEXT("signedpak")) || FParse::Param(FCommandLine::Get(), TEXT("signed"));
#if !UE_BUILD_SHIPPING
		bShouldCheckSignature &= !FParse::Param(FCommandLine::Get(), TEXT("FileOpenLog"));
#endif
		if (bShouldCheckSignature)
		{
			bNeedsDecryptor = true;
		}			
	}

	if(bNeedsDecryptor && !Decryptor.IsValid())
	{
		TUniquePtr<FArchive> DecryptorReader{ MakeArchive() };
		if (DecryptorReader.IsValid())
		{
			Decryptor = MakeUnique<FChunkCacheWorker>(MoveTemp(DecryptorReader), Filename);
		}

		if (!Decryptor.IsValid() || !Decryptor->IsValid())
		{
			return nullptr;
		}
	}

	// Now we either have a Decryptor or we don't need it
	check(!bNeedsDecryptor || Decryptor.IsValid());

	TUniquePtr<FArchive> Archive{ MakeArchive() };
	if (!Archive.IsValid())
	{
		return nullptr;
	}

	if (Decryptor.IsValid())
	{
		return new FSignedArchiveReader(Archive.Release(), Decryptor.Get());
	}
	else
	{
		return Archive.Release();
	}
}

void FPakFile::Initialize(FArchive& Reader, bool bLoadIndex)
{
	CachedTotalSize = Reader.TotalSize();
	bool bShouldLoad = false;
	int32 CompatibleVersion = FPakInfo::PakFile_Version_Latest;

	LLM_SCOPE_BYNAME(TEXT("FileSystem/PakFile"));

	// Serialize trailer and check if everything is as expected.
	// start up one to offset the -- below
	CompatibleVersion++;
	int64 FileInfoPos = -1;
	do
	{
		// try the next version down
		CompatibleVersion--;

		FileInfoPos = CachedTotalSize - Info.GetSerializedSize(CompatibleVersion);
		if (FileInfoPos >= 0)
		{
			Reader.Seek(FileInfoPos);
			Reader.Precache(FileInfoPos, 0); // Inform the archive that we're going to repeatedly serialize from the current location

			SCOPED_BOOT_TIMING("PakFile_SerilizeTrailer");

			// Serialize trailer and check if everything is as expected.
			Info.Serialize(Reader, CompatibleVersion);
			if (Info.Magic == FPakInfo::PakFile_Magic)
			{
				bShouldLoad = true;
			}
		}
	} while (!bShouldLoad && CompatibleVersion >= FPakInfo::PakFile_Version_Initial);

	if (bShouldLoad)
	{
		UE_CLOGF(Info.Magic != FPakInfo::PakFile_Magic, LogPakFile, Fatal, "Trailing magic number (%ud) in '%ls' is different than the expected one. Verify your installation.", Info.Magic, *PakFilename);
		UE_CLOG(!(Info.Version >= FPakInfo::PakFile_Version_Initial && Info.Version <= CompatibleVersion), LogPakFile, Fatal, TEXT("Invalid pak file version (%d) in '%s'. Verify your installation."), Info.Version, *PakFilename);
		UE_CLOG(!(Info.IndexOffset >= 0 && Info.IndexOffset < CachedTotalSize), LogPakFile, Fatal, TEXT("Index offset for pak file '%s' is invalid (%lld is bigger than file size %lld)"), *PakFilename, Info.IndexOffset, CachedTotalSize);
		UE_CLOG(!((Info.IndexOffset + Info.IndexSize) >= 0 && (Info.IndexOffset + Info.IndexSize) <= CachedTotalSize), LogPakFile, Fatal, TEXT("Index end offset for pak file '%s' is invalid (%lld)"), *PakFilename, Info.IndexOffset + Info.IndexSize);

		// If we aren't using a dynamic encryption key, process the pak file using the embedded key
		if (!Info.EncryptionKeyGuid.IsValid() || UE::FEncryptionKeyManager::Get().ContainsKey(Info.EncryptionKeyGuid))
		{
			if (bLoadIndex)
			{
				LoadIndex(Reader);
			}

			if (ShouldCheckPak())
			{
				ensure(Check());
			}
		}

		if (Decryptor.IsValid())
		{
			TSharedPtr<const FPakSignatureFile, ESPMode::ThreadSafe> SignatureFile = Decryptor->GetSignatures();
			if (SignatureFile->SignatureData.Num() == UE_ARRAY_COUNT(FSHAHash::Hash))
			{
				bIsValid = (FMemory::Memcmp(SignatureFile->SignatureData.GetData(), Info.IndexHash.Hash, SignatureFile->SignatureData.Num()) == 0);
			}
			else
			{
				bIsValid = false;
			}
		}
		else
		{
			// LoadIndex should crash in case of an error, so just assume everything is ok if we got here.
			bIsValid = true;
		}
	}
}

void FPakFile::LoadIndex(FArchive& Reader)
{
#if ENABLE_PAKFILE_USE_DIRECTORY_TREE
	bUseDirectoryTree = GPak_UseDirectoryTreeForPakSearch;
	bool bStoreDirectoryTree = bUseDirectoryTree;
	bool bStoreDirectoryTMap = !bUseDirectoryTree;
#if !UE_BUILD_SHIPPING
	bStoreDirectoryTMap |= GPak_ValidateDirectoryTreeSearchConsistency;
#endif // !UE_BUILD_SHIPPING
#else // !ENABLE_PAKFILE_USE_DIRECTORY_TREE
	constexpr bool bStoreDirectoryTMap = true;
	constexpr bool bStoreDirectoryTree = false;
#endif // else !ENABLE_PAKFILE_USE_DIRECTORY_TREE

	if (Info.Version >= FPakInfo::PakFile_Version_PathHashIndex)
	{
		if (!LoadIndexInternal(Reader, DirectoryIndex, DirectoryTreeIndex, PrunedDirectoryIndex, PrunedDirectoryTreeIndex,
			bStoreDirectoryTMap, bStoreDirectoryTree))
		{
			// Index loading failed. Try again
			if (!LoadIndexInternal(Reader, DirectoryIndex, DirectoryTreeIndex, PrunedDirectoryIndex, PrunedDirectoryTreeIndex,
				bStoreDirectoryTMap, bStoreDirectoryTree))
			{
				UE_LOGF(LogPakFile, Fatal, "Corrupt pak index detected on pak file: %ls", *PakFilename);
			}
		}
	}
	else
	{
		SCOPED_BOOT_TIMING("PakFile_LoadLegacy");
		FDirectoryIndex LoadedTMap;
		if (!LoadLegacyIndex(Reader, LoadedTMap))
		{
			// Index loading failed. Try again
			if (!LoadLegacyIndex(Reader, LoadedTMap))
			{
				UE_LOGF(LogPakFile, Fatal, "Corrupt pak index detected on pak file: %ls", *PakFilename);
			}
		}
#if ENABLE_PAKFILE_USE_DIRECTORY_TREE
		if (bStoreDirectoryTree)
		{
			for (TMap<FString, FPakDirectory>::TIterator It(LoadedTMap); It; ++It)
			{
				DirectoryTreeIndex.FindOrAdd(It->Key) =
					bStoreDirectoryTMap ? FPakDirectory(It->Value) : MoveTemp(It->Value);
			}
			DirectoryTreeIndex.Shrink();
		}
#endif
		if (bStoreDirectoryTMap)
		{
			DirectoryIndex = MoveTemp(LoadedTMap);
		}
	}
}

bool FPakFile::LoadIndexInternal(FArchive& Reader, FDirectoryIndex& OutDirectoryTMap,
	FDirectoryTreeIndex& OutDirectoryTree, FDirectoryIndex& OutPrunedDirectoryTMap,
	FDirectoryTreeIndex& OutPrunedDirectoryTree, bool bStoreDirectoryTMap, bool bStoreDirectoryTree)
{
	bHasPathHashIndex = false;
	bHasFullDirectoryIndex = false;
#if ENABLE_PAKFILE_RUNTIME_PRUNING
	bNeedsLegacyPruning = false;
	bWillPruneDirectoryIndex = false;
#endif

	FGuardedInt64 IndexEndPosition = FGuardedInt64(Info.IndexOffset) + Info.IndexSize;
	if (Info.IndexOffset < 0 || 
		Info.IndexSize < 0 ||
		IndexEndPosition.InvalidOrGreaterThan(CachedTotalSize) ||
		IntFitsIn<int32>(Info.IndexSize) == false)
	{
		UE_LOGF(LogPakFile, Fatal, "Corrupted index offset in pak file.");
		return false;
	}

	TArray<uint8> PrimaryIndexData;
	Reader.Seek(Info.IndexOffset);
	PrimaryIndexData.SetNum((int32)(Info.IndexSize));
	{
		SCOPED_BOOT_TIMING("PakFile_LoadPrimaryIndex");
		Reader.Serialize(PrimaryIndexData.GetData(), Info.IndexSize);
	}

	FSHAHash ComputedHash;
	{
		SCOPED_BOOT_TIMING("PakFile_HashPrimaryIndex");
		if (!DecryptAndValidateIndex(Reader, PrimaryIndexData, Info.IndexHash, ComputedHash))
		{
			UE_LOGF(LogPakFile, Log, "Corrupt pak PrimaryIndex detected!");
			UE_LOGF(LogPakFile, Log, " Filename: %ls", *PakFilename);
			UE_LOGF(LogPakFile, Log, " Encrypted: %d", Info.bEncryptedIndex);
			UE_LOGF(LogPakFile, Log, " Total Size: %lld", Reader.TotalSize());
			UE_LOGF(LogPakFile, Log, " Index Offset: %lld", Info.IndexOffset);
			UE_LOGF(LogPakFile, Log, " Index Size: %lld", Info.IndexSize);
			UE_LOGF(LogPakFile, Log, " Stored Index Hash: %ls", *Info.IndexHash.ToString());
			UE_LOGF(LogPakFile, Log, " Computed Index Hash: %ls", *ComputedHash.ToString());
			return false;
		}
	}

	FMemoryReader PrimaryIndexReader(PrimaryIndexData);

	// Read the scalar data (mount point, numentries, etc) and all entries.
	NumEntries = 0;
	PrimaryIndexReader << MountPoint;
	// We are just deserializing a string, which could get however long. Since we know it's a path
	// and paths are bound to file system rules, we know it can't get absurdly long (e.g. windows is _at best_ 32k)
	// we just sanity check it to prevent operating on massive buffers and risking overflows.
	if (MountPoint.Len() > 65535)
	{
		UE_LOGF(LogPakFile, Error, "Corrupt pak index data: MountPoint path is longer than 65k");
		return false;
	}

	MakeDirectoryFromPath(MountPoint);
	PrimaryIndexReader << NumEntries;
	if (NumEntries < 0)
	{
		UE_LOGF(LogPakFile, Error, "Corrupt pak index data: Negative entries count in pak file.");
		return false;
	}
	PrimaryIndexReader << PathHashSeed;

	bool bReaderHasPathHashIndex = false;
	int64 PathHashIndexOffset = INDEX_NONE;
	int64 PathHashIndexSize = 0;
	FSHAHash PathHashIndexHash;
	PrimaryIndexReader << bReaderHasPathHashIndex;
	if (bReaderHasPathHashIndex)
	{
		PrimaryIndexReader << PathHashIndexOffset;
		PrimaryIndexReader << PathHashIndexSize;
		PrimaryIndexReader << PathHashIndexHash;
		bReaderHasPathHashIndex = bReaderHasPathHashIndex && PathHashIndexOffset != INDEX_NONE;
	}

	bool bReaderHasFullDirectoryIndex = false;
	int64 FullDirectoryIndexOffset = INDEX_NONE;
	int64 FullDirectoryIndexSize = 0;
	FSHAHash FullDirectoryIndexHash;
	PrimaryIndexReader << bReaderHasFullDirectoryIndex;
	if (bReaderHasFullDirectoryIndex)
	{
		PrimaryIndexReader << FullDirectoryIndexOffset;
		PrimaryIndexReader << FullDirectoryIndexSize;
		PrimaryIndexReader << FullDirectoryIndexHash;
		bReaderHasFullDirectoryIndex = bReaderHasFullDirectoryIndex && FullDirectoryIndexOffset  != INDEX_NONE;
	}
	{
		SCOPED_BOOT_TIMING("PakFile_SerializeEncodedEntries");
		PrimaryIndexReader << EncodedPakEntries;
	}

	int32 FilesNum = 0;
	PrimaryIndexReader << FilesNum;
	if (FilesNum < 0)
	{
		// Should not be possible for any values in the PrimaryIndex to be invalid, since we verified the index hash
		UE_LOGF(LogPakFile, Log, "Corrupt pak PrimaryIndex detected!");
		UE_LOGF(LogPakFile, Log, " FilesNum: %d", FilesNum);
		return false;
	}
	Files.SetNum(FilesNum);
	if (FilesNum > 0)
	{
		SCOPED_BOOT_TIMING("PakFile_SerializeUnencodedEntries");
		FPakEntry* FileEntries = Files.GetData();
		for (int32 FileIndex = 0; FileIndex < FilesNum; ++FileIndex)
		{
			FileEntries[FileIndex].Serialize(PrimaryIndexReader, Info.Version);
		}
	}

	// Decide which SecondaryIndex(es) to load
	bool bWillUseFullDirectoryIndex;
	bool bWillUsePathHashIndex;
	bool bReadFullDirectoryIndex;
	if (bReaderHasPathHashIndex && bReaderHasFullDirectoryIndex)
	{
		bWillUseFullDirectoryIndex = IsPakKeepFullDirectory();
		bWillUsePathHashIndex = !bWillUseFullDirectoryIndex;
#if ENABLE_PAKFILE_RUNTIME_PRUNING
		bool bWantToReadFullDirectoryIndex = IsPakKeepFullDirectory() || IsPakValidatePruning() || IsPakDelayPruning();
#else
		bool bWantToReadFullDirectoryIndex = IsPakKeepFullDirectory();
#endif
		bReadFullDirectoryIndex = bReaderHasFullDirectoryIndex && bWantToReadFullDirectoryIndex;
	}
	else if (bReaderHasPathHashIndex)
	{
		bWillUsePathHashIndex = true;
		bWillUseFullDirectoryIndex = false;
		bReadFullDirectoryIndex = false;
	}
	else if (bReaderHasFullDirectoryIndex)
	{
		// We don't support creating the PathHash Index at runtime; we want to move to having only the PathHashIndex, so supporting not having it at all is not useful enough to write
		bWillUsePathHashIndex = false;
		bWillUseFullDirectoryIndex = true;
		bReadFullDirectoryIndex = true;
	}
	else
	{
		// It should not be possible for PrimaryIndexes to be built without a PathHashIndex AND without a FullDirectoryIndex; CreatePakFile in UnrealPak.exe has a check statement for it.
		UE_LOGF(LogPakFile, Log, "Corrupt pak PrimaryIndex detected!");
		UE_LOGF(LogPakFile, Log, " bReaderHasPathHashIndex: false");
		UE_LOGF(LogPakFile, Log, " bReaderHasFullDirectoryIndex: false");
		return false;
	}

	// Load the Secondary Index(es)
	TArray<uint8> PathHashIndexData;
	FMemoryReader PathHashIndexReader(PathHashIndexData);
	if (bWillUsePathHashIndex)
	{
		FGuardedInt64 PathHashIndexEndPosition = FGuardedInt64(PathHashIndexOffset) + PathHashIndexSize;
		if (PathHashIndexOffset < 0 || 
			PathHashIndexSize < 0 ||
			PathHashIndexEndPosition.InvalidOrGreaterThan(CachedTotalSize) || 
			IntFitsIn<int32>(PathHashIndexSize) == false)
		{
			// Should not be possible for these values (which came from the PrimaryIndex) to be invalid, since we verified the index hash of the PrimaryIndex
			UE_LOGF(LogPakFile, Log, "Corrupt pak PrimaryIndex detected!");
			UE_LOGF(LogPakFile, Log, " Filename: %ls", *PakFilename);
			UE_LOGF(LogPakFile, Log, " Total Size: %lld", Reader.TotalSize());
			UE_LOGF(LogPakFile, Log, " PathHashIndexOffset : %lld", PathHashIndexOffset);
			UE_LOGF(LogPakFile, Log, " PathHashIndexSize: %lld", PathHashIndexSize);
			return false;
		}
		Reader.Seek(PathHashIndexOffset);
		PathHashIndexData.SetNum((int32)(PathHashIndexSize));
		{
			SCOPED_BOOT_TIMING("PakFile_LoadPathHashIndex");
			Reader.Serialize(PathHashIndexData.GetData(), PathHashIndexSize);
		}

		{
			SCOPED_BOOT_TIMING("PakFile_HashPathHashIndex");
			if (!DecryptAndValidateIndex(Reader, PathHashIndexData, PathHashIndexHash, ComputedHash))
			{
				UE_LOGF(LogPakFile, Log, "Corrupt pak PathHashIndex detected!");
				UE_LOGF(LogPakFile, Log, " Filename: %ls", *PakFilename);
				UE_LOGF(LogPakFile, Log, " Encrypted: %d", Info.bEncryptedIndex);
				UE_LOGF(LogPakFile, Log, " Total Size: %lld", Reader.TotalSize());
				UE_LOGF(LogPakFile, Log, " Index Offset: %lld", FullDirectoryIndexOffset);
				UE_LOGF(LogPakFile, Log, " Index Size: %lld", FullDirectoryIndexSize);
				UE_LOGF(LogPakFile, Log, " Stored Index Hash: %ls", *PathHashIndexHash.ToString());
				UE_LOGF(LogPakFile, Log, " Computed Index Hash: %ls", *ComputedHash.ToString());
				return false;
			}
		}

		{
			SCOPED_BOOT_TIMING("PakFile_SerializePathHashIndex");
			PathHashIndexReader << PathHashIndex;
		}
		bHasPathHashIndex = true;
	}
	
	if (!bReadFullDirectoryIndex)
	{
		check(bWillUsePathHashIndex); // Need to confirm that we have read the PathHashIndex bytes
		// Store the PrunedDirectoryIndex in our DirectoryIndex
		{
			SCOPED_BOOT_TIMING("PakFile_SerializePrunedDirectoryIndex");
			LoadIndexInternal_DirectoryIndex(PathHashIndexReader, OutDirectoryTMap, OutDirectoryTree,
				bStoreDirectoryTMap, bStoreDirectoryTree);
		}
		bHasFullDirectoryIndex = false;
#if ENABLE_PAKFILE_RUNTIME_PRUNING
		bWillPruneDirectoryIndex = false;
#endif
	}
	else
	{
		FGuardedInt64 FullDirectoryIndexEndPosition = FGuardedInt64(FullDirectoryIndexOffset) + FullDirectoryIndexSize;
		if (FullDirectoryIndexOffset  < 0 || 
			FullDirectoryIndexSize < 0 ||
			FullDirectoryIndexEndPosition.InvalidOrGreaterThan(CachedTotalSize) || 
			IntFitsIn<int32>(FullDirectoryIndexSize) == false)
		{
			// Should not be possible for these values (which came from the PrimaryIndex) to be invalid, since we verified the index hash of the PrimaryIndex
			UE_LOGF(LogPakFile, Log, "Corrupt pak PrimaryIndex detected!");
			UE_LOGF(LogPakFile, Log, " Filename: %ls", *PakFilename);
			UE_LOGF(LogPakFile, Log, " Total Size: %lld", Reader.TotalSize());
			UE_LOGF(LogPakFile, Log, " FullDirectoryIndexOffset : %lld", FullDirectoryIndexOffset );
			UE_LOGF(LogPakFile, Log, " FullDirectoryIndexSize: %lld", FullDirectoryIndexSize);
			return false;
		}
		TArray<uint8> FullDirectoryIndexData;
		Reader.Seek(FullDirectoryIndexOffset );
		FullDirectoryIndexData.SetNum((int32)(FullDirectoryIndexSize));
		{
			SCOPED_BOOT_TIMING("PakFile_LoadDirectoryIndex");
			Reader.Serialize(FullDirectoryIndexData.GetData(), FullDirectoryIndexSize);
		}

		{
			SCOPED_BOOT_TIMING("PakFile_HashDirectoryIndex");
			if (!DecryptAndValidateIndex(Reader, FullDirectoryIndexData, FullDirectoryIndexHash, ComputedHash))
			{
				UE_LOGF(LogPakFile, Log, "Corrupt pak FullDirectoryIndex detected!");
				UE_LOGF(LogPakFile, Log, " Filename: %ls", *PakFilename);
				UE_LOGF(LogPakFile, Log, " Encrypted: %d", Info.bEncryptedIndex);
				UE_LOGF(LogPakFile, Log, " Total Size: %lld", Reader.TotalSize());
				UE_LOGF(LogPakFile, Log, " Index Offset: %lld", FullDirectoryIndexOffset);
				UE_LOGF(LogPakFile, Log, " Index Size: %lld", FullDirectoryIndexSize);
				UE_LOGF(LogPakFile, Log, " Stored Index Hash: %ls", *FullDirectoryIndexHash.ToString());
				UE_LOGF(LogPakFile, Log, " Computed Index Hash: %ls", *ComputedHash.ToString());
				return false;
			}
		}

		FMemoryReader SecondaryIndexReader(FullDirectoryIndexData);
		{
			SCOPED_BOOT_TIMING("PakFile_SerializeDirectoryIndex");
			LoadIndexInternal_DirectoryIndex(SecondaryIndexReader, OutDirectoryTMap, OutDirectoryTree,
				bStoreDirectoryTMap, bStoreDirectoryTree);
		}
		bHasFullDirectoryIndex = true;

#if ENABLE_PAKFILE_RUNTIME_PRUNING
		if (bWillUseFullDirectoryIndex)
		{
			bWillPruneDirectoryIndex = false;
		}
		else
		{
			// Store the PrunedDirectoryIndex from the PrimaryIndex in our PrunedDirectoryIndex, to be used for verification and to be swapped into DirectoryIndex later
			check(bWillUsePathHashIndex); // Need to confirm that we have read the PathHashIndex bytes
			{
				SCOPED_BOOT_TIMING("PakFile_SerializePrunedDirectoryIndex");
				LoadIndexInternal_DirectoryIndex(PathHashIndexReader, OutPrunedDirectoryTMap, OutPrunedDirectoryTree,
					bStoreDirectoryTMap, bStoreDirectoryTree);
			}
			bWillPruneDirectoryIndex = true;
			bSomePakNeedsPruning = true;
		}
#endif
	}

	UE_LOG(LogPakFile, Verbose, TEXT("PakFile PrimaryIndexSize=%" INT64_FMT), Info.IndexSize);
	UE_LOG(LogPakFile, Verbose, TEXT("PakFile PathHashIndexSize=%" INT64_FMT), PathHashIndexSize);
	UE_LOG(LogPakFile, Verbose, TEXT("PakFile FullDirectoryIndexSize=%" INT64_FMT), FullDirectoryIndexSize);

	check(bHasFullDirectoryIndex || bHasPathHashIndex);
	return true;
}

void FPakFile::LoadIndexInternal_DirectoryIndex(FArchive& Ar, FDirectoryIndex& OutDirectoryTMap,
	FDirectoryTreeIndex& OutDirectoryTree, bool bLoadIntoDirectoryTMap, bool bLoadIntoDirectoryTree)
{
	FString FileName;
	FString DirectoryName;
	FPakEntryLocation FileData;
	FPakDirectory DirectoryData;

	int32 NumDirectories = 0;
	Ar << NumDirectories;
	OutDirectoryTMap.Reset();
	if (bLoadIntoDirectoryTMap)
	{
		OutDirectoryTMap.Reserve(NumDirectories);
	}
#if ENABLE_PAKFILE_USE_DIRECTORY_TREE
	OutDirectoryTree.Empty();
#endif
	for (; NumDirectories > 0; --NumDirectories)
	{
		Ar << DirectoryName;
		int32 NumFiles = 0;
		Ar << NumFiles;

		DirectoryData.Reset();
		DirectoryData.Reserve(NumFiles);

		if (Info.Version >= FPakInfo::PakFile_Version_Utf8PakDirectory)
		{
			FUtf8String FileNameUtf8;
			for (; NumFiles > 0; --NumFiles)
			{
				Ar << FileNameUtf8;
				Ar << FileData;
				DirectoryData.Add(MoveTemp(FileNameUtf8), FileData);
			}
		}
		else
		{
			for (; NumFiles > 0; --NumFiles)
			{
				Ar << FileName;
				Ar << FileData;
				DirectoryData.Add(FUtf8String(FileName), FileData);
			}
		}

#if ENABLE_PAKFILE_USE_DIRECTORY_TREE
		if (bLoadIntoDirectoryTree)
		{
			OutDirectoryTree.FindOrAdd(DirectoryName) =
				bLoadIntoDirectoryTMap ? FPakDirectory(DirectoryData) : MoveTemp(DirectoryData);

		}
#endif
		if (bLoadIntoDirectoryTMap)
		{
			OutDirectoryTMap.FindOrAdd(DirectoryName) = MoveTemp(DirectoryData);
		}
	}
#if ENABLE_PAKFILE_USE_DIRECTORY_TREE
	OutDirectoryTree.Shrink();
#endif
}

void FPakFile::SaveIndexInternal_DirectoryIndex(FArchive& Ar, const FDirectoryIndex& DirectoryTMap)
{
	int32 NumDirectories = DirectoryTMap.Num();
	Ar << NumDirectories;
	for (const TPair<FString, FPakDirectory>& DirPair : DirectoryTMap)
	{
		Ar << const_cast<FString&>(DirPair.Key);
		int32 NumFiles = DirPair.Value.Num();
		Ar << NumFiles;

		for (const TPair<FUtf8String, FPakEntryLocation>& FilePair : DirPair.Value)
		{
			Ar << const_cast<FUtf8String&>(FilePair.Key);
			Ar << const_cast<FPakEntryLocation&>(FilePair.Value);
		}
	}
}

bool FPakFile::LoadLegacyIndex(FArchive& Reader, FDirectoryIndex& OutDirectoryTMap)
{
	bHasPathHashIndex = false;
	bHasFullDirectoryIndex = false;
#if ENABLE_PAKFILE_RUNTIME_PRUNING
	bNeedsLegacyPruning = false;
	bWillPruneDirectoryIndex = false;
#endif

	// Load index into memory first.
	FGuardedInt64 IndexEndPosition = FGuardedInt64(Info.IndexOffset) + Info.IndexSize;
	if (Info.IndexSize < 0 ||
		Info.IndexOffset < 0 ||
		IndexEndPosition.InvalidOrGreaterThan(CachedTotalSize) ||
		IntFitsIn<int32>(Info.IndexSize) == false)
	{
		UE_LOGF(LogPakFile, Fatal, "Corrupted index offset/size in pak file.");
		return false;
	}

	TArray<uint8> IndexData;
	IndexData.SetNum((int32)(Info.IndexSize));

	Reader.Seek(Info.IndexOffset);
	Reader.Serialize(IndexData.GetData(), Info.IndexSize);

	FSHAHash ComputedHash;
	if (!DecryptAndValidateIndex(Reader, IndexData, Info.IndexHash, ComputedHash))
	{
		UE_LOGF(LogPakFile, Log, "Corrupt pak index detected!");
		UE_LOGF(LogPakFile, Log, " Filename: %ls", *PakFilename);
		UE_LOGF(LogPakFile, Log, " Encrypted: %d", Info.bEncryptedIndex);
		UE_LOGF(LogPakFile, Log, " Total Size: %lld", Reader.TotalSize());
		UE_LOGF(LogPakFile, Log, " Index Offset: %lld", Info.IndexOffset);
		UE_LOGF(LogPakFile, Log, " Index Size: %lld", Info.IndexSize);
		UE_LOGF(LogPakFile, Log, " Stored Index Hash: %ls", *Info.IndexHash.ToString());
		UE_LOGF(LogPakFile, Log, " Computed Index Hash: %ls", *ComputedHash.ToString());
		return false;
	}


	FMemoryReader IndexReader(IndexData);

	// Read the default mount point and all entries.
	NumEntries = 0;
	IndexReader << MountPoint;

	// We are just deserializing a string, which could get however long. Since we know it's a path
	// and paths are bound to file system rules, we know it can't get absurdly long (e.g. windows is _at best_ 32k)
	// we just sanity check it to prevent operating on massive buffers and risking overflows.
	if (MountPoint.Len() > 65535)
	{
		UE_LOGF(LogPakFile, Error, "Corrupt pak index data: MountPoint path is longer than 65k");
		return false;
	}
	IndexReader << NumEntries;

	if (NumEntries < 0)
	{
		UE_LOGF(LogPakFile, Error, "Corrupt pak index data: NumEntries is negative");
		return false;
	}

	MakeDirectoryFromPath(MountPoint);

	FPakEntryPair PakEntryPair;
	auto ReadNextEntry = [&PakEntryPair, &IndexReader, this]() -> FPakEntryPair&
	{
		IndexReader << PakEntryPair.Filename;
		PakEntryPair.Info.Reset();
		PakEntryPair.Info.Serialize(IndexReader, this->Info.Version);
		return PakEntryPair;
	};

	TMap<uint64, FString> CollisionDetection;
	int32 NumEncodedEntries = 0;
	int32 NumDeletedEntries = 0;
#if ENABLE_PAKFILE_RUNTIME_PRUNING
	bool bCreatePathHash = !IsPakKeepFullDirectory();
#else
	// Pruning of legacy files is no longer supported; we will keep the entire directory with no way to prune it.  There is no need to create the PathHashIndex since we will have the FullDirectoryIndex.
	bool bCreatePathHash = false;
#endif
	FPathHashIndex* PathHashToWrite = bCreatePathHash ? &PathHashIndex : nullptr;
	FPakFile::EncodePakEntriesIntoIndex(NumEntries, ReadNextEntry, *PakFilename, Info, MountPoint, NumEncodedEntries, NumDeletedEntries, &PathHashSeed,
		&OutDirectoryTMap, PathHashToWrite, EncodedPakEntries, Files, &CollisionDetection, Info.Version);
	check(NumEncodedEntries + Files.Num() + NumDeletedEntries == NumEntries);
	Files.Shrink();
	EncodedPakEntries.Shrink();

	bHasPathHashIndex = bCreatePathHash;
	bHasFullDirectoryIndex = true;
#if ENABLE_PAKFILE_RUNTIME_PRUNING
	if (!IsPakKeepFullDirectory())
	{
		bNeedsLegacyPruning = true;
		bWillPruneDirectoryIndex = true;
		bSomePakNeedsPruning = true;
		// We cannot prune during this call because config settings have not yet been loaded and we need the settings for DirectoryIndexKeepFiles before we can prune
		// PrunedDirectoryIndex will be created and swapped with DirectoryIndex in OptimizeMemoryUsageForMountedPaks, and bHasFullDirectoryIndex will be set to false then
	}
	else
	{
		bNeedsLegacyPruning = false;
		bWillPruneDirectoryIndex = false;
	}
#endif

	check(bHasFullDirectoryIndex || bHasPathHashIndex);
	return true;
}

bool FPakFile::DecryptAndValidateIndex(FArchive& Reader, TArray<uint8>& IndexData, FSHAHash& InExpectedHash, FSHAHash& OutActualHash)
{
	// Decrypt if necessary
	if (Info.bEncryptedIndex)
	{
		DecryptData(IndexData.GetData(), IndexData.Num(), Info.EncryptionKeyGuid);
	}

	// Check SHA1 value.
	FSHA1::HashBuffer(IndexData.GetData(), IndexData.Num(), OutActualHash.Hash);
	return InExpectedHash == OutActualHash;
}

uint64 FPakFile::HashPath(const TCHAR* RelativePathFromMount, uint64 Seed, int32 PakFileVersion)
{
	FString LowercaseRelativePath(RelativePathFromMount);
	LowercaseRelativePath.ToLowerInline();
	if (PakFileVersion >= FPakInfo::PakFile_Version_Fnv64BugFix)
	{
		return FFnv::MemFnv64(*LowercaseRelativePath, LowercaseRelativePath.Len() * sizeof(TCHAR), Seed);
	}
	else
	{
		return LegacyMemFnv64(*LowercaseRelativePath, LowercaseRelativePath.Len() * sizeof(TCHAR), Seed);
	}
}

void FPakFile::EncodePakEntriesIntoIndex(int32 InNumEntries, const ReadNextEntryFunction& InReadNextEntry, const TCHAR* InPakFilename, const FPakInfo& InPakInfo, const FString& MountPoint,
	int32& OutNumEncodedEntries, int32& OutNumDeletedEntries, uint64* OutPathHashSeed,
	FDirectoryIndex* OutDirectoryIndex, FPathHashIndex* OutPathHashIndex, TArray<uint8>& OutEncodedPakEntries, TArray<FPakEntry>& OutNonEncodableEntries,
	TMap<uint64, FString>* InOutCollisionDetection, int32 PakFileVersion)
{
	uint64 PathHashSeed = 0;
	if (OutPathHashSeed || OutPathHashIndex)
	{
		FString LowercasePakFilename(InPakFilename);
		LowercasePakFilename.ToLowerInline();
		PathHashSeed = FCrc::StrCrc32(*LowercasePakFilename);
		if (OutPathHashSeed)
		{
			*OutPathHashSeed = PathHashSeed;
		}
	}

	OutNumEncodedEntries = 0;
	OutNumDeletedEntries = 0;
	FMemoryWriter CompressedEntryWriter(OutEncodedPakEntries);
	for (int32 EntryCount = 0; EntryCount < InNumEntries; ++EntryCount)
	{
		FPakEntryPair& Pair = InReadNextEntry();
		// Add the Entry and get an FPakEntryLocation for it
		const FPakEntry& PakEntry = Pair.Info;
		FPakEntryLocation EntryLocation;
		if (!PakEntry.IsDeleteRecord())
		{
			EntryLocation = FPakEntryLocation::CreateFromOffsetIntoEncoded(OutEncodedPakEntries.Num());
			if (EncodePakEntry(CompressedEntryWriter, PakEntry, InPakInfo))
			{
				++OutNumEncodedEntries;
			}
			else
			{
				int32 ListIndex = OutNonEncodableEntries.Num();
				EntryLocation = FPakEntryLocation::CreateFromListIndex(ListIndex);
				OutNonEncodableEntries.Add(PakEntry);

				// PakEntries in the index have some values that are different from the in-place pakentries stored next to each file's payload.  EncodePakEntry handles that internally if encoding succeeded.
				FPakEntry& StoredPakEntry = OutNonEncodableEntries[ListIndex];
				FMemory::Memset(StoredPakEntry.Hash, 0); // Hash is 0-filled
				StoredPakEntry.Verified = true; // Validation of the hash is impossible, so Verified is set to true
			}
		}
		else
		{
			++OutNumDeletedEntries;
		}

		// Add the Entry into the requested Indexes
		AddEntryToIndex(Pair.Filename, EntryLocation, MountPoint, PathHashSeed, OutDirectoryIndex,
			nullptr /* DirectoryTreeIndex */, OutPathHashIndex, InOutCollisionDetection, PakFileVersion);
	}
}

void FPakFile::PruneDirectoryIndex(FDirectoryIndex& InOutDirectoryIndex, FDirectoryIndex* InPrunedDirectoryIndex,
	const FString& MountPoint)
{
	PruneDirectoryIndexInternal(&InOutDirectoryIndex, nullptr, InPrunedDirectoryIndex, nullptr, MountPoint);
}

void FPakFile::PruneDirectoryIndexInternal(FDirectoryIndex* InOutDirectoryIndex, FDirectoryTreeIndex* InOutDirectoryTreeIndex,
	FDirectoryIndex* InPrunedDirectoryIndex, FDirectoryTreeIndex* InPrunedDirectoryTreeIndex, const FString& MountPoint)
{
	// Caller holds WriteLock on DirectoryIndexLock
	TArray<FString> FileWildCards, DirectoryWildCards, OldWildCards;
	GConfig->GetArray(TEXT("Pak"), TEXT("DirectoryIndexKeepFiles"), FileWildCards, GEngineIni);
	GConfig->GetArray(TEXT("Pak"), TEXT("DirectoryIndexKeepEmptyDirectories"), DirectoryWildCards, GEngineIni);
	GConfig->GetArray(TEXT("Pak"), TEXT("DirectoryRootsToKeepInMemoryWhenUnloadingPakEntryFilenames"), OldWildCards, GEngineIni); // Legacy name, treated as both KeepFiles and KeepEmptyDirectories
	DirectoryWildCards.Append(OldWildCards);
	FileWildCards.Append(OldWildCards);
	int32 NumKeptEntries = 0;

	if (InPrunedDirectoryIndex)
	{
		InPrunedDirectoryIndex->Empty();
	}
#if ENABLE_PAKFILE_USE_DIRECTORY_TREE
	if (InPrunedDirectoryTreeIndex)
	{
		InPrunedDirectoryTreeIndex->Empty();
	}
#endif

	TMap<FString, bool> KeepDirectory;

	// Clear out those portions of the Index allowed by the user.
	if (DirectoryWildCards.Num() > 0 || FileWildCards.Num() > 0)
	{
		if (InOutDirectoryIndex)
		{
			for (FDirectoryIndex::TIterator It = InOutDirectoryIndex->CreateIterator(); It; ++It)
			{
				const FString& DirectoryName = It.Key();
				FPakDirectory& OriginalDirectory = It.Value();
				const FString FullDirectoryName = PakPathCombine(MountPoint, DirectoryName);
				check(IsPathInDirectoryFormat(FullDirectoryName));
				FPakDirectory* PrunedDirectory = nullptr;
				bool bKeepDirectory = false;

				TArray<FUtf8String> RemoveFilenames;
				for (auto FileIt = It->Value.CreateIterator(); FileIt; ++FileIt)
				{
					const FUtf8String& FileNameWithoutPath = FileIt->Key;
					const FString FullFilename = PakPathCombine(FullDirectoryName, FileNameWithoutPath);
					bool bKeepFile = false;

					for (const FString& WildCard : FileWildCards)
					{
						if (FullFilename.MatchesWildcard(WildCard))
						{
							bKeepFile = true;
							break;
						}
					}

					if (bKeepFile)
					{
						bKeepDirectory = true;
						if (InPrunedDirectoryIndex)
						{
							if (!PrunedDirectory)
							{
								PrunedDirectory = &InPrunedDirectoryIndex->Add(DirectoryName);
							}
							PrunedDirectory->Add(FileNameWithoutPath, FileIt->Value);
						}
					}
					else
					{
						if (!InPrunedDirectoryIndex)
						{
							RemoveFilenames.Add(FileNameWithoutPath);
						}
					}
				}
				if (!InPrunedDirectoryIndex)
				{
					for (const FUtf8String& FileNameWithoutPath : RemoveFilenames)
					{
						OriginalDirectory.Remove(FileNameWithoutPath);
					}
				}

				if (!bKeepDirectory)
				{
					for (const FString& WildCard : DirectoryWildCards)
					{
						if (FullDirectoryName.MatchesWildcard(WildCard))
						{
							bKeepDirectory = true;
							break;
						}
					}
				}
				KeepDirectory.FindOrAdd(DirectoryName) = bKeepDirectory;
			}

			{
				// For each kept directory, mark that we need to keep all of its parents up to the mount point.
				// Note: We rely on TMap reallocations for KeepDirectory not modifying the underlying FString data,
				// so the FStringViews in KeptDirectories remain valid.
				TArray<FStringView> KeptDirectories;
				for (const TPair<FString, bool>& Pair : KeepDirectory)
				{
					if (Pair.Value)
					{
						KeptDirectories.Add(Pair.Key);
					}
				}

				for (const FStringView& KeptDirectory : KeptDirectories)
				{
					FStringView CurrentDirectory = KeptDirectory;
					FStringView UnusedCleanFileName;
					while (CurrentDirectory != MountPoint)
					{
						if (!SplitPathInline(CurrentDirectory, UnusedCleanFileName))
						{
							break;
						}
						const uint32 CurrentDirectoryHash = GetTypeHash(CurrentDirectory);
						bool* bOldValue = KeepDirectory.FindByHash(CurrentDirectoryHash, CurrentDirectory);
						if (!bOldValue)
						{
							bOldValue = &KeepDirectory.AddByHash(CurrentDirectoryHash, FString(CurrentDirectory));
						}
						if (*bOldValue)
						{
							break;
						}
						*bOldValue = true;
					}
				}
			}

			// Prune all of the directories
			for (const TPair<FString, bool>& Pair : KeepDirectory)
			{
				const FString& DirectoryName = Pair.Key;
				bool bKeep = Pair.Value;
				if (bKeep)
				{
					if (InPrunedDirectoryIndex)
					{
						InPrunedDirectoryIndex->FindOrAdd(DirectoryName);
					}
				}
				else
				{
					if (!InPrunedDirectoryIndex)
					{
						InOutDirectoryIndex->Remove(DirectoryName);
					}
				}
			}
		}
#if ENABLE_PAKFILE_USE_DIRECTORY_TREE
		if (InOutDirectoryTreeIndex)
		{
			for (FDirectoryTreeIndex::FIterator It(InOutDirectoryTreeIndex->CreateIterator()); It; ++It)
			{
				FString DirectoryName = FString(It->Key);
				FPakDirectory& OriginalDirectory = It->Value;
				FString FullDirectoryName = PakPathCombine(MountPoint, DirectoryName);
				if (!FullDirectoryName.EndsWith(TEXT("/")))
				{
					FullDirectoryName += '/';
				}
				check(IsPathInDirectoryFormat(FullDirectoryName));
				FPakDirectory* PrunedDirectory = nullptr;
				bool bKeepDirectory = false;

				TArray<FUtf8String> RemoveFilenames;
				for (FPakDirectory::TIterator FileIt = OriginalDirectory.CreateIterator(); FileIt; ++FileIt)
				{
					const FUtf8String& FileNameWithoutPath = FileIt->Key;
					const FString FullFilename = PakPathCombine(FullDirectoryName, FileNameWithoutPath);
					bool bKeepFile = false;

					for (const FString& WildCard : FileWildCards)
					{
						if (FullFilename.MatchesWildcard(WildCard))
						{
							bKeepFile = true;
							break;
						}
					}

					if (bKeepFile)
					{
						bKeepDirectory = true;
						if (InPrunedDirectoryTreeIndex)
						{
							if (!PrunedDirectory)
							{
								PrunedDirectory = &InPrunedDirectoryTreeIndex->FindOrAdd(DirectoryName);
							}
							PrunedDirectory->Add(FileNameWithoutPath, FileIt->Value);
						}
					}
					else
					{
						if (!InPrunedDirectoryTreeIndex)
						{
							RemoveFilenames.Add(FileNameWithoutPath);
						}
					}
				}
				if (!InPrunedDirectoryTreeIndex)
				{
					for (const FUtf8String& FileNameWithoutPath : RemoveFilenames)
					{
						OriginalDirectory.Remove(FileNameWithoutPath);
					}
				}

				if (!bKeepDirectory)
				{
					for (const FString& WildCard : DirectoryWildCards)
					{
						if (FullDirectoryName.MatchesWildcard(WildCard))
						{
							bKeepDirectory = true;
							break;
						}
					}
				}
				KeepDirectory.FindOrAdd(DirectoryName) = bKeepDirectory;
			}

			{
				// For each kept directory, mark that we need to keep all of its parents up to the mount point.
				// Note: We rely on TMap reallocations for KeepDirectory not modifying the underlying FString data,
				// so the FStringViews in KeptDirectories remain valid.
				TArray<FStringView> KeptDirectories;
				for (const TPair<FString, bool>& Pair : KeepDirectory)
				{
					if (Pair.Value)
					{
						KeptDirectories.Add(Pair.Key);
					}
				}

				for (const FStringView& KeptDirectory : KeptDirectories)
				{
					FStringView CurrentDirectory = KeptDirectory;
					FStringView UnusedCleanFileName;
					while (CurrentDirectory != MountPoint)
					{
						if (!SplitPathInline(CurrentDirectory, UnusedCleanFileName))
						{
							break;
						}
						const uint32 CurrentDirectoryHash = GetTypeHash(CurrentDirectory);
						bool* bOldValue = KeepDirectory.FindByHash(CurrentDirectoryHash, CurrentDirectory);
						if (!bOldValue)
						{
							bOldValue = &KeepDirectory.AddByHash(CurrentDirectoryHash, FString(CurrentDirectory));
						}
						if (*bOldValue)
						{
							break;
						}
						*bOldValue = true;
					}
				}
			}

			// Prune all of the directories
			for (const TPair<FString, bool>& Pair : KeepDirectory)
			{
				const FString& DirectoryName = Pair.Key;
				bool bKeep = Pair.Value;
				if (bKeep)
				{
					if (InPrunedDirectoryTreeIndex)
					{
						InPrunedDirectoryTreeIndex->FindOrAdd(DirectoryName);
					}
				}
				else
				{
					if (!InPrunedDirectoryTreeIndex)
					{
						InOutDirectoryIndex->Remove(DirectoryName);
					}
				}
			}
		}
#endif
	}
	else
	{
		if (InOutDirectoryIndex)
		{
			if (!InPrunedDirectoryIndex)
			{
				InOutDirectoryIndex->Empty();
			}
		}
#if ENABLE_PAKFILE_USE_DIRECTORY_TREE
		if (InOutDirectoryTreeIndex)
		{
			if (!InPrunedDirectoryTreeIndex)
			{
				InOutDirectoryTreeIndex->Empty();
			}
		}
#endif
	}
}

template <typename CharType>
FString PakPathCombineInternal(FStringView Parent, TStringView<CharType> Child)
{
	// Our paths are different than FPaths, because our dirs / at the end, and "/" is the relative path to the mountdirectory and should be mapped to the empty string when joining
	check(Parent.Len() > 0 && Parent[Parent.Len() - 1] == TEXT('/'));
	if (Parent.Len() == 1)
	{
		return FString(Child);
	}
	else if (Child.Len() == 1 && Child[0] == CharType('/'))
	{
		return FString(Parent);
	}
	else
	{
		check(Child.Len() == 0 || Child[0] != CharType('/'));
		FString Result;
		Result.Reserve(Parent.Len() + Child.Len());
		Result += Parent;
		Result += Child;
		return Result;
	}
}

FString FPakFile::PakPathCombine(FStringView Parent, FStringView Child)
{
	return PakPathCombineInternal(Parent, Child);
}

FString FPakFile::PakPathCombine(FStringView Parent, FUtf8StringView Child)
{
	return PakPathCombineInternal(Parent, Child);
}

bool FPakFile::SplitPathInline(FStringView& InOutPath, FStringView& OutFilename)
{
	// FPaths::GetPath doesn't handle our / at the end of directories, so we have to do string manipulation ourselves
	// The manipulation is less complicated than GetPath deals with, since we have normalized/path/strings, we have relative paths only, and we don't care about extensions
	if (InOutPath.Len() == 0)
	{
		check(false); // Filenames should have non-zero length, and the minimum directory length is 1 (The root directory is written as "/")
		return false;
	}
	else if (InOutPath.Len() == 1)
	{
		if (InOutPath[0] == TEXT('/'))
		{
			// The root directory; it has no parent.
			OutFilename = FStringView();
			return false;
		}
		else
		{
			// A relative one-character path with no /; this is a direct child of in the root directory
			OutFilename = TEXT("/");
			Swap(OutFilename, InOutPath);
			return true;
		}
	}
	else
	{
		if (InOutPath[InOutPath.Len() - 1] == TEXT('/'))
		{
			// The input was a Directory; remove the trailing / since we don't keep those on the CleanFilename
			InOutPath.LeftChopInline(1);
		}

		int32 Offset = 0;
		if (InOutPath.FindLastChar(TEXT('/'), Offset))
		{
			int32 FilenameStart = Offset + 1;
			OutFilename = InOutPath.Mid(FilenameStart);
			InOutPath.LeftInline(FilenameStart); // The Parent Directory keeps the / at the end
		}
		else
		{
			// A relative path with no /; this is a direct child of in the root directory
			OutFilename = TEXT("/");
			Swap(OutFilename, InOutPath);
		}
		return true;
	}
}

FPakFile::EFindResult FPakFile::GetPakEntry(const FPakEntryLocation& PakEntryLocation, FPakEntry* OutEntry) const
{
	return GetPakEntry(PakEntryLocation, OutEntry, EncodedPakEntries, Files, Info);
}

FPakFile::EFindResult FPakFile::GetPakEntry(const FPakEntryLocation& PakEntryLocation, FPakEntry* OutEntry, const TArray<uint8>& EncodedPakEntries, const TArray<FPakEntry>& Files, const FPakInfo& Info)
{
	bool bDeleted = PakEntryLocation.IsInvalid();
	if (OutEntry != NULL)
	{
		if (!bDeleted)
		{
			// The FPakEntry structures are bit-encoded, so decode it.
			int32 EncodedOffset = PakEntryLocation.GetAsOffsetIntoEncoded();
			if (EncodedOffset >= 0)
			{
				check(EncodedOffset < EncodedPakEntries.Num());
				DecodePakEntry(EncodedPakEntries.GetData() + EncodedOffset, *OutEntry, Info);
			}
			else
			{
				int32 ListIndex = PakEntryLocation.GetAsListIndex();
				check(ListIndex >= 0);
				const FPakEntry& FoundEntry = Files[ListIndex];
				//*OutEntry = **FoundEntry;
				OutEntry->Offset = FoundEntry.Offset;
				OutEntry->Size = FoundEntry.Size;
				OutEntry->UncompressedSize = FoundEntry.UncompressedSize;
				OutEntry->CompressionMethodIndex = FoundEntry.CompressionMethodIndex;
				OutEntry->CompressionBlocks = FoundEntry.CompressionBlocks;
				OutEntry->CompressionBlockSize = FoundEntry.CompressionBlockSize;
				OutEntry->Flags = FoundEntry.Flags;
			}
		}
		else
		{
			// entry was deleted, build dummy entry to indicate the deleted entry
			(*OutEntry) = FPakEntry();
			OutEntry->SetDeleteRecord(true);
		}

		// Index PakEntries do not store their hash, so verification against the hash is impossible.
		// Initialize the OutEntry's Hash to 0 and mark it as already verified.
		// TODO: Verified and Hash are checked by FPakFileHandle, which is used when opening files from PakFiles synchronously;
		//       it is not currently used by asynchronous reads in FPakPrecacher, and we can likely remove it from use in FPakFileHandle as well
		FMemory::Memset(OutEntry->Hash, 0);
		OutEntry->Verified = true;
	}

	return bDeleted ? EFindResult::FoundDeleted : EFindResult::Found;
}

FPakFile::FIndexSettings& FPakFile::GetIndexSettings()
{
	static FIndexSettings IndexLoadParams;
	return IndexLoadParams;
}

bool FPakFile::IsPakKeepFullDirectory()
{
	FIndexSettings& IndexLoadParams = GetIndexSettings();
	return IndexLoadParams.bKeepFullDirectory;
}

bool FPakFile::IsPakValidatePruning()
{
#if ENABLE_PAKFILE_RUNTIME_PRUNING_VALIDATE
	FIndexSettings& IndexLoadParams = GetIndexSettings();
	return IndexLoadParams.bValidatePruning;
#else
	return false;
#endif
}

bool FPakFile::IsPakDelayPruning()
{
	FIndexSettings& IndexLoadParams = GetIndexSettings();
	return IndexLoadParams.bDelayPruning;
}

bool FPakFile::IsPakWritePathHashIndex()
{
	FIndexSettings& IndexLoadParams = GetIndexSettings();
	return IndexLoadParams.bWritePathHashIndex;
}

bool FPakFile::IsPakWriteFullDirectoryIndex()
{
	FIndexSettings& IndexLoadParams = GetIndexSettings();
	return IndexLoadParams.bWriteFullDirectoryIndex;
}

bool FPakFile::RequiresDirectoryIndexLock() const
{
#if ENABLE_PAKFILE_RUNTIME_PRUNING
	return bWillPruneDirectoryIndex;
#else
	return false; 
#endif
}

bool FPakFile::ShouldValidatePrunedDirectory() const
{
#if ENABLE_PAKFILE_RUNTIME_PRUNING_VALIDATE
	return IsPakValidatePruning() && bWillPruneDirectoryIndex && !bNeedsLegacyPruning;
#else
	return false;
#endif
}

bool FPakFile::ShouldUseDirectoryTree() const
{
#if ENABLE_PAKFILE_USE_DIRECTORY_TREE
	return bUseDirectoryTree;
#else
	return false;
#endif
}

void FPakFile::AddEntryToIndex(const FString& Filename, const FPakEntryLocation& EntryLocation,
	const FString& MountPoint, uint64 PathHashSeed, FDirectoryIndex* DirectoryIndex,
	FDirectoryTreeIndex* DirectoryTreeIndex, FPathHashIndex* PathHashIndex, TMap<uint64, FString>* CollisionDetection,
	int32 PakFileVersion)
{
	FString RelativePathFromMount;
	if (FPaths::IsRelative(Filename))
	{
		RelativePathFromMount = Filename;
	}
	else
	{
		check(IsPathInDirectoryFormat(MountPoint));
		RelativePathFromMount = Filename;
		check(Filename.Len() > MountPoint.Len());
		bool bSucceeded = GetRelativePathFromMountInline(RelativePathFromMount, MountPoint);
		check(bSucceeded);
	}

	if (DirectoryIndex)
	{
		FStringView RelativeDirectoryFromMount = RelativePathFromMount;
		FStringView CleanFileName;
		SplitPathInline(RelativeDirectoryFromMount, CleanFileName);
		const uint32 RelativeDirectoryFromMountHash = GetTypeHash(RelativeDirectoryFromMount);
		FPakDirectory* Directory = DirectoryIndex->FindByHash(RelativeDirectoryFromMountHash, RelativeDirectoryFromMount);
		if (Directory == nullptr)
		{
			// add the parent directories up to the mount point (mount point relative path is "/")
			FStringView CurrentDirectory(RelativeDirectoryFromMount);
			FStringView UnusedCleanFileName;
			while (!CurrentDirectory.IsEmpty())
			{
				if (!SplitPathInline(CurrentDirectory, UnusedCleanFileName))
				{
					break;
				}
				const uint32 CurrentDirectoryHash = GetTypeHash(CurrentDirectory);
				if (!DirectoryIndex->FindByHash(CurrentDirectoryHash, CurrentDirectory))
				{
					DirectoryIndex->AddByHash(CurrentDirectoryHash, FString(CurrentDirectory));
				}
			}

			// Add the new directory; this has to come after the addition of the parent directories because we want to use the pointer afterwards and adding other directories would invalidate it
			Directory = &DirectoryIndex->AddByHash(RelativeDirectoryFromMountHash, FString(RelativeDirectoryFromMount));
		}
		Directory->Add(FUtf8String(CleanFileName), EntryLocation);
	}
#if ENABLE_PAKFILE_USE_DIRECTORY_TREE
	if (DirectoryTreeIndex)
	{
		FStringView RelativeDirectoryFromMount = RelativePathFromMount;
		FStringView CleanFileName;
		SplitPathInline(RelativeDirectoryFromMount, CleanFileName);
		FPakDirectory* Directory = DirectoryTreeIndex->Find(RelativeDirectoryFromMount);
		if (Directory == nullptr)
		{
			// add the parent directories up to the mount point (mount point relative path is "/")
			FStringView CurrentDirectory(RelativeDirectoryFromMount);
			FStringView UnusedCleanFileName;
			while (!CurrentDirectory.IsEmpty())
			{
				if (!SplitPathInline(CurrentDirectory, UnusedCleanFileName))
				{
					break;
				}
				DirectoryTreeIndex->FindOrAdd(CurrentDirectory);
			}

			// Add the new directory; this has to come after the addition of the parent directories because we want to use the pointer afterwards and adding other directories would invalidate it
			Directory = &DirectoryTreeIndex->FindOrAdd(RelativeDirectoryFromMount);
		}
		Directory->Add(FUtf8String(CleanFileName), EntryLocation);
	}
#endif

	// Add the entry into the PathHash index
	if (CollisionDetection || PathHashIndex)
	{
		uint64 PathHash = FPakFile::HashPath(*RelativePathFromMount, PathHashSeed, PakFileVersion);
		if (CollisionDetection)
		{
			FString* ExistingFilename = CollisionDetection->Find(PathHash);
			checkf(!ExistingFilename || ExistingFilename->Equals(RelativePathFromMount), TEXT("Hash collision for two Filenames within a PakFile.  Filename1 == '%s'.  Filename2 == '%s'.  Hash='0x%" UINT64_x_FMT "'.")
				TEXT(" Collision handling is not yet implemented; to resolve the conflict, modify one of the Filenames."), **ExistingFilename, *RelativePathFromMount, PathHash);
			CollisionDetection->Add(PathHash, RelativePathFromMount);
		}
		if (PathHashIndex)
		{
			PathHashIndex->Add(PathHash, EntryLocation);
		}
	}
}

// Take a pak entry and byte encode it into a smaller representation
bool FPakFile::EncodePakEntry(FArchive& Ar, const FPakEntry& InPakEntry, const FPakInfo& InInfo)
{
	// See notes in DecodePakEntry for the output layout

	check(Ar.IsSaving()); // This function is encode only, and promises not to modify InEntry, but we want to use << which takes non-const so we need to assert that Ar is a loader not a saver
	check(!InPakEntry.IsDeleteRecord()); // Deleted PakEntries cannot be encoded, caller must check for IsDeleteRecord and handle it separately by e.g. not adding the Entry to the FileList and instead giving the referencer an invalid FPakEntryLocation
	FPakEntry& PakEntry = const_cast<FPakEntry&>(InPakEntry);

	const uint32 CompressedBlockAlignment = PakEntry.IsEncrypted() ? FAES::AESBlockSize : 1;
	const int64 HeaderSize = PakEntry.GetSerializedSize(InInfo.Version);

	// This data fits into a bitfield (described in DecodePakEntry), and the data has
	// to fit within a certain range of bits.
	if (PakEntry.CompressionMethodIndex >= (1 << 6))
	{
		return false;
	}
	if (PakEntry.CompressionBlocks.Num() >= (1 << 16))
	{
		return false;
	}
	if (PakEntry.CompressionMethodIndex != 0)
	{
		if (PakEntry.CompressionBlocks.Num() > 0 && ((InInfo.HasRelativeCompressedChunkOffsets() ? 0 : PakEntry.Offset) + HeaderSize != PakEntry.CompressionBlocks[0].CompressedStart))
		{
			return false;
		}
		if (PakEntry.CompressionBlocks.Num() == 1)
		{
			uint64 Base = InInfo.HasRelativeCompressedChunkOffsets() ? 0 : PakEntry.Offset;
			uint64 AlignedBlockSize = Align(PakEntry.CompressionBlocks[0].CompressedEnd - PakEntry.CompressionBlocks[0].CompressedStart, CompressedBlockAlignment);
			if ((Base + HeaderSize + PakEntry.Size) != (PakEntry.CompressionBlocks[0].CompressedStart + AlignedBlockSize))
			{
				return false;
			}
		}
		if (PakEntry.CompressionBlocks.Num() > 1)
		{
			for (int i = 1; i < PakEntry.CompressionBlocks.Num(); ++i)
			{
				uint64 PrevBlockSize = PakEntry.CompressionBlocks[i - 1].CompressedEnd - PakEntry.CompressionBlocks[i - 1].CompressedStart;
				PrevBlockSize = Align(PrevBlockSize, CompressedBlockAlignment);
				if (PakEntry.CompressionBlocks[i].CompressedStart != (PakEntry.CompressionBlocks[i - 1].CompressedStart + PrevBlockSize))
				{
					return false;
				}
			}
		}
	}

	// This entry can be encoded, so let's do it!

	bool bIsOffset32BitSafe = PakEntry.Offset <= MAX_uint32;
	bool bIsSize32BitSafe = PakEntry.Size <= MAX_uint32;
	bool bIsUncompressedSize32BitSafe = PakEntry.UncompressedSize <= MAX_uint32;
	
	// If CompressionBlocks.Num() == 1, we set CompressionBlockSize == UncompressedSize and record CompressBlockSizePacked=0
	// Otherwise, we encode CompressionBlockSize as a 6-bit multiple of 1 << 11.
	// If CompressionBlockSize is not a multiple of 1 << 11, or is a larger multiple than 6 bits we can not encode correctly.
	// In that case we set the packed field to its maximum value (0x3F) and send the unencoded CompressionBlockSize as a separate value.
	uint32 CompressionBlockSizePacked = 0;
	if (PakEntry.CompressionBlocks.Num() > 1)
	{
		CompressionBlockSizePacked = (PakEntry.CompressionBlockSize >> 11) & 0x3F;
		if ((CompressionBlockSizePacked << 11) != PakEntry.CompressionBlockSize)
		{
			CompressionBlockSizePacked = 0x3F;
		}
	}

	// Build the Flags field.
	uint32 Flags =
		(bIsOffset32BitSafe ? (1 << 31) : 0)
		| (bIsUncompressedSize32BitSafe ? (1 << 30) : 0)
		| (bIsSize32BitSafe ? (1 << 29) : 0)
		| (PakEntry.CompressionMethodIndex << 23)
		| (PakEntry.IsEncrypted() ? (1 << 22) : 0)
		| (PakEntry.CompressionBlocks.Num() << 6)
		| CompressionBlockSizePacked
		;

	Ar << Flags;
	
	// if we write 0x3F for CompressionBlockSize then send the field
	if ( CompressionBlockSizePacked == 0x3F )
	{
		uint32 Value = (uint32)PakEntry.CompressionBlockSize;
		Ar << Value;
	}

	// Build the Offset field.
	if (bIsOffset32BitSafe)
	{
		uint32 Value = (uint32)PakEntry.Offset;
		Ar << Value;
	}
	else
	{
		Ar << PakEntry.Offset;
	}

	// Build the Uncompressed Size field.
	if (bIsUncompressedSize32BitSafe)
	{
		uint32 Value = (uint32)PakEntry.UncompressedSize;
		Ar << Value;
	}
	else
	{
		Ar << PakEntry.UncompressedSize;
	}

	// Any additional data is for compressed file data.
	if (PakEntry.CompressionMethodIndex != 0)
	{
		// Build the Compressed Size field.
		if (bIsSize32BitSafe)
		{
			uint32 Value = (uint32)PakEntry.Size;
			Ar << Value;
		}
		else
		{
			Ar << PakEntry.Size;
		}

		// Build the Compression Blocks array.
		if (PakEntry.CompressionBlocks.Num() > 1 || (PakEntry.CompressionBlocks.Num() == 1 && PakEntry.IsEncrypted()))
		{
			for (int CompressionBlockIndex = 0; CompressionBlockIndex < PakEntry.CompressionBlocks.Num(); ++CompressionBlockIndex)
			{
				uint32 Value = IntCastChecked<uint32>(PakEntry.CompressionBlocks[CompressionBlockIndex].CompressedEnd - PakEntry.CompressionBlocks[CompressionBlockIndex].CompressedStart);
				Ar << Value;
			}
		}
	}

	return true;
}

void FPakFile::DecodePakEntry(const uint8* SourcePtr, FPakEntry& OutEntry, const FPakInfo& InInfo)
{
	// Grab the big bitfield value:
	// Bit 31 = Offset 32-bit safe?
	// Bit 30 = Uncompressed size 32-bit safe?
	// Bit 29 = Size 32-bit safe?
	// Bits 28-23 = Compression method
	// Bit 22 = Encrypted
	// Bits 21-6 = Compression blocks count
	// Bits 5-0 = Compression block size
	uint32 Value = *(uint32*)SourcePtr;
	SourcePtr += sizeof(uint32);
	
	uint32 CompressionBlockSize = 0;
	if ( (Value & 0x3f) == 0x3f ) // flag value to load a field
	{
		CompressionBlockSize = *(uint32*)SourcePtr;
		SourcePtr += sizeof(uint32);
	}
	else
	{
		// for backwards compatibility with old paks :
		CompressionBlockSize = ((Value & 0x3f) << 11);
	}

	// Filter out the CompressionMethod.
	OutEntry.CompressionMethodIndex = (Value >> 23) & 0x3f;

	// Test for 32-bit safe values. Grab it, or memcpy the 64-bit value
	// to avoid alignment exceptions on platforms requiring 64-bit alignment
	// for 64-bit variables.
	//
	// Read the Offset.
	bool bIsOffset32BitSafe = (Value & (1 << 31)) != 0;
	if (bIsOffset32BitSafe)
	{
		OutEntry.Offset = *(uint32*)SourcePtr;
		SourcePtr += sizeof(uint32);
	}
	else
	{
		FMemory::Memcpy(&OutEntry.Offset, SourcePtr, sizeof(int64));
		SourcePtr += sizeof(int64);
	}

	// Read the UncompressedSize.
	bool bIsUncompressedSize32BitSafe = (Value & (1 << 30)) != 0;
	if (bIsUncompressedSize32BitSafe)
	{
		OutEntry.UncompressedSize = *(uint32*)SourcePtr;
		SourcePtr += sizeof(uint32);
	}
	else
	{
		FMemory::Memcpy(&OutEntry.UncompressedSize, SourcePtr, sizeof(int64));
		SourcePtr += sizeof(int64);
	}

	// Fill in the Size.
	if (OutEntry.CompressionMethodIndex != 0)
	{
		// Size is only present if compression is applied.
		bool bIsSize32BitSafe = (Value & (1 << 29)) != 0;
		if (bIsSize32BitSafe)
		{
			OutEntry.Size = *(uint32*)SourcePtr;
			SourcePtr += sizeof(uint32);
		}
		else
		{
			FMemory::Memcpy(&OutEntry.Size, SourcePtr, sizeof(int64));
			SourcePtr += sizeof(int64);
		}
	}
	else
	{
		// The Size is the same thing as the UncompressedSize when
		// CompressionMethod == COMPRESS_None.
		OutEntry.Size = OutEntry.UncompressedSize;
	}

	// Filter the encrypted flag.
	OutEntry.SetEncrypted((Value & (1 << 22)) != 0);

	// This should clear out any excess CompressionBlocks that may be valid in the user's
	// passed in entry.
	uint32 CompressionBlocksCount = (Value >> 6) & 0xffff;
	OutEntry.CompressionBlocks.Empty(CompressionBlocksCount);
	OutEntry.CompressionBlocks.SetNum(CompressionBlocksCount);
	
	OutEntry.CompressionBlockSize = 0;
	if (CompressionBlocksCount > 0)
	{
		OutEntry.CompressionBlockSize = CompressionBlockSize;
		// Per the comment in Encode, if CompressionBlocksCount == 1, we use UncompressedSize for CompressionBlockSize
		if (CompressionBlocksCount == 1)
		{
			OutEntry.CompressionBlockSize = IntCastChecked<uint32>(OutEntry.UncompressedSize);
		}
		ensure(OutEntry.CompressionBlockSize != 0);
	}

	// Set bDeleteRecord to false, because it obviously isn't deleted if we are here.
	OutEntry.SetDeleteRecord(false);

	// Base offset to the compressed data
	int64 BaseOffset = InInfo.HasRelativeCompressedChunkOffsets() ? 0 : OutEntry.Offset;

	// Handle building of the CompressionBlocks array.
	if (OutEntry.CompressionBlocks.Num() == 1 && !OutEntry.IsEncrypted())
	{
		// If the number of CompressionBlocks is 1, we didn't store any extra information.
		// Derive what we can from the entry's file offset and size.
		FPakCompressedBlock& CompressedBlock = OutEntry.CompressionBlocks[0];
		CompressedBlock.CompressedStart = BaseOffset + OutEntry.GetSerializedSize(InInfo.Version);
		CompressedBlock.CompressedEnd = CompressedBlock.CompressedStart + OutEntry.Size;
	}
	else if (OutEntry.CompressionBlocks.Num() > 0)
	{
		// Get the right pointer to start copying the CompressionBlocks information from.
		uint32* CompressionBlockSizePtr = (uint32*)SourcePtr;

		// Alignment of the compressed blocks
		uint64 CompressedBlockAlignment = OutEntry.IsEncrypted() ? FAES::AESBlockSize : 1;

		// CompressedBlockOffset is the starting offset. Everything else can be derived from there.
		int64 CompressedBlockOffset = BaseOffset + OutEntry.GetSerializedSize(InInfo.Version);
		for (int CompressionBlockIndex = 0; CompressionBlockIndex < OutEntry.CompressionBlocks.Num(); ++CompressionBlockIndex)
		{
			FPakCompressedBlock& CompressedBlock = OutEntry.CompressionBlocks[CompressionBlockIndex];
			CompressedBlock.CompressedStart = CompressedBlockOffset;
			CompressedBlock.CompressedEnd = CompressedBlockOffset + *CompressionBlockSizePtr++;
			CompressedBlockOffset += Align(CompressedBlock.CompressedEnd - CompressedBlock.CompressedStart, CompressedBlockAlignment);
		}
	}
}

bool FPakFile::NormalizeDirectoryQuery(const TCHAR* InPath, FString& OutRelativePathFromMount) const
{
	OutRelativePathFromMount = InPath;
	MakeDirectoryFromPath(OutRelativePathFromMount);
	return GetRelativePathFromMountInline(OutRelativePathFromMount, MountPoint);
}

const FPakDirectory* FPakFile::FindPrunedDirectoryInternal(const FString& RelativePathFromMount) const
{
	// Caller holds FScopedPakDirectoryIndexAccess
	const FPakDirectory* PakDirectory = nullptr;

#if ENABLE_PAKFILE_RUNTIME_PRUNING_VALIDATE
	if (ShouldValidatePrunedDirectory())
	{
		PakDirectory = FindPrunedDirectoryInIndexInternal(RelativePathFromMount, DirectoryIndex, DirectoryTreeIndex);
		const FPakDirectory* PrunedPakDirectory = FindPrunedDirectoryInIndexInternal(RelativePathFromMount,
			PrunedDirectoryIndex, PrunedDirectoryTreeIndex);
		if ((PakDirectory != nullptr) != (PrunedPakDirectory != nullptr))
		{
			TSet<FString> FullFoundFiles, PrunedFoundFiles;
			FString ReportedDirectoryName = MountPoint + RelativePathFromMount;
			if (PakDirectory)
			{
				FullFoundFiles.Add(ReportedDirectoryName);
			}
			if (PrunedPakDirectory)
			{
				PrunedFoundFiles.Add(ReportedDirectoryName);
			}
			ValidateDirectorySearch(FullFoundFiles, PrunedFoundFiles, *ReportedDirectoryName);
		}
	}
	else
#endif
	{
		PakDirectory = FindPrunedDirectoryInIndexInternal(RelativePathFromMount, DirectoryIndex, DirectoryTreeIndex);
	}
	return PakDirectory;
}

const FPakDirectory* FPakFile::FindPrunedDirectoryInIndexInternal(const FString& RelativePathFromMount,
	const FDirectoryIndex& InDirectoryIndex, const FDirectoryTreeIndex& InTreeIndex) const
{
	const FPakDirectory* PakDirectory = nullptr;
#if ENABLE_PAKFILE_USE_DIRECTORY_TREE
	if (ShouldUseDirectoryTree())
	{
		PakDirectory = InTreeIndex.Find(RelativePathFromMount);
#if !UE_BUILD_SHIPPING
		if (GPak_ValidateDirectoryTreeSearchConsistency)
		{
			const FPakDirectory* IndexedResult = InDirectoryIndex.Find(RelativePathFromMount);
			if ((PakDirectory != nullptr) != (IndexedResult != nullptr) ||
				(PakDirectory && PakDirectory->Num() != IndexedResult->Num()))
			{
				UE_LOGF(LogPakFile, Fatal, "Mismatch between directoryindex and directorytreeindex search when searching for [%ls] in pak [%ls]",
					*FString(RelativePathFromMount), *GetFilename());
			}
		}
#endif // !UE_BUILD_SHIPPING
	}
	else
#endif // ENABLE_PAKFILE_USE_DIRECTORY_TREE
	{
		PakDirectory = InDirectoryIndex.Find(RelativePathFromMount);
	}
	return PakDirectory;
}

bool FPakFile::Check()
{
	UE_LOGF(LogPakFile, Display, "Checking pak file \"%ls\". This may take a while...", *PakFilename);
	double StartTime = FPlatformTime::Seconds();

	FSharedPakReader PakReader = GetSharedReader(nullptr);
	int32 ErrorCount = 0;
	int32 FileCount = 0;

	// If the pak file is signed, we can do a fast check by just reading a single byte from the start of
	// each signing block. The signed archive reader will bring in that whole signing block and compare
	// against the signature table and fire the handler
	if (bSigned)
	{
		FDelegateHandle DelegateHandle;
		FPakPlatformFile::FPakSigningFailureHandlerData& HandlerData = FPakPlatformFile::GetPakSigningFailureHandlerData();

		{
			FScopeLock Lock(&HandlerData.GetLock());
			DelegateHandle = HandlerData.GetPakChunkSignatureCheckFailedDelegate().AddLambda([&ErrorCount](const FPakChunkSignatureCheckFailedData&)
			{
				++ErrorCount;
			});
		}

		int64 CurrentPos = 0;
		const int64 Size = PakReader->TotalSize();
		while (CurrentPos < Size)
		{
			PakReader->Seek(CurrentPos);
			uint8 Byte = 0;
			PakReader.GetArchive() << Byte;
			CurrentPos += FPakInfo::MaxChunkDataSize;
		}

		if (DelegateHandle.IsValid())
		{
			FScopeLock Lock(&HandlerData.GetLock());
			HandlerData.GetPakChunkSignatureCheckFailedDelegate().Remove(DelegateHandle);
		}
	}
	else
	{
		const bool bIncludeDeleted = true;
		TCHAR EntryNameBuffer[256];
		auto GetEntryName = [&EntryNameBuffer](const FPakFile::FPakEntryIterator& It)
		{
			const FString* EntryFilename = It.TryGetFilename();
			if (EntryFilename)
			{
				TCString<TCHAR>::Snprintf(EntryNameBuffer, sizeof(EntryNameBuffer), TEXT("\"%s\""), **EntryFilename);
			}
			else
			{
				TCString<TCHAR>::Snprintf(EntryNameBuffer, sizeof(EntryNameBuffer), TEXT("file at offset %u"), It.Info().Offset);
			}
			return EntryNameBuffer;
		};
		for (FPakFile::FPakEntryIterator It(*this, bIncludeDeleted); It; ++It, ++FileCount)
		{
			const FPakEntry& EntryFromIndex = It.Info();
			if (EntryFromIndex.IsDeleteRecord())
			{
				UE_LOGF(LogPakFile, Verbose, "%ls Deleted.", GetEntryName(It));
				continue;
			}

			void* FileContents = FMemory::Malloc(EntryFromIndex.Size);
			PakReader->Seek(EntryFromIndex.Offset);
			uint32 SerializedCrcTest = 0;
			FPakEntry EntryFromPayload;
			EntryFromPayload.Serialize(PakReader.GetArchive(), GetInfo().Version);
			if (!EntryFromPayload.IndexDataEquals(EntryFromIndex))
			{
				UE_LOGF(LogPakFile, Error, "Index FPakEntry does not match Payload FPakEntry for %ls.", GetEntryName(It));
				ErrorCount++;
			}
			PakReader->Serialize(FileContents, EntryFromIndex.Size);

			uint8 TestHash[20];
			FSHA1::HashBuffer(FileContents, EntryFromIndex.Size, TestHash);
			if (FMemory::Memcmp(TestHash, EntryFromPayload.Hash, sizeof(TestHash)) != 0)
			{
				UE_LOGF(LogPakFile, Error, "Hash mismatch for %ls.", GetEntryName(It));
				ErrorCount++;
			}
			else
			{
				UE_LOGF(LogPakFile, Verbose, "%ls OK. [%ls]", GetEntryName(It), *Info.GetCompressionMethod(EntryFromIndex.CompressionMethodIndex).ToString());
			}
			FMemory::Free(FileContents);
		}
		if (ErrorCount == 0)
		{
			UE_LOGF(LogPakFile, Display, "Pak file \"%ls\" healthy, %d files checked.", *PakFilename, FileCount);
		}
		else
		{
			UE_LOGF(LogPakFile, Display, "Pak file \"%ls\" corrupted (%d errors out of %d files checked.).", *PakFilename, ErrorCount, FileCount);
		}
	}

	double EndTime = FPlatformTime::Seconds();
	double ElapsedTime = EndTime - StartTime;
	UE_LOGF(LogPakFile, Display, "Pak file \"%ls\" checked in %.2fs", *PakFilename, ElapsedTime);

	return ErrorCount == 0;
}

void FPakFile::GetPrunedFilenames(TArray<FString>& OutFileList) const
{
	for (FFilenameIterator It(*this, true /* bIncludeDeleted */); It; ++It)
	{
		OutFileList.Add(PakPathCombine(MountPoint, It.Filename()));
	}
}

void FPakFile::GetPrunedFilenamesInChunk(const TArray<int32>& InChunkIDs, TArray<FString>& OutFileList) const
{
	for (FFilenameIterator It(*this, true /* bIncludeDeleted */); It; ++It)
	{
		const FPakEntry& File = It.Info();
		int64 FileStart = File.Offset;
		int64 FileEnd = File.Offset + File.Size;

		for (int64 LocalChunkID : InChunkIDs)
		{
			int64 ChunkStart = LocalChunkID * FPakInfo::MaxChunkDataSize;
			int64 ChunkEnd = ChunkStart + FPakInfo::MaxChunkDataSize;

			if (FileStart < ChunkEnd && FileEnd > ChunkStart)
			{
				OutFileList.Add(It.Filename());
				break;
			}
		}
	}
}

void FPakFile::FindPrunedFilesAtPath(const TCHAR* InPath, TArray<FString>& OutFiles,
	bool bIncludeFiles, bool bIncludeDirectories, bool bRecursive) const
{
	auto ShouldVisit = [](FStringView Path) { return true; };
	FindPrunedFilesAtPathInternal(InPath, OutFiles, FVisitFilter(ShouldVisit, bIncludeFiles, bIncludeDirectories, bRecursive));
}


#if ENABLE_PAKFILE_RUNTIME_PRUNING_VALIDATE
void FPakFile::ValidateDirectorySearch(const TSet<FString>& FullFoundFiles, const TSet<FString>& PrunedFoundFiles, const TCHAR* InPath) const
{
	TArray<FString> MissingFromPruned;
	for (const FString& FileInFull : FullFoundFiles)
	{
		if (!PrunedFoundFiles.Contains(FileInFull))
		{
			MissingFromPruned.Add(FileInFull);
		}
	}
	TArray<FString> MissingFromFull;
	for (const FString& FileInPruned : PrunedFoundFiles)
	{
		if (!FullFoundFiles.Contains(FileInPruned))
		{
			MissingFromFull.Add(FileInPruned);
		}
	}

	if (MissingFromPruned.Num() == 0 && MissingFromFull.Num() == 0)
	{
		return;
	}

	TArray<FString> WildCards, OldWildCards;
	GConfig->GetArray(TEXT("Pak"), TEXT("IndexValidationIgnore"), WildCards, GEngineIni);
	auto IsIgnore = [&WildCards](const FString& FilePath)
	{
		for (const FString& WildCard : WildCards)
		{
			if (FilePath.MatchesWildcard(WildCard))
			{
				return true;
			}
		}
		return false;
	};
	auto StripIgnores = [&IsIgnore](TArray<FString>& FilePaths)
	{
		for (int Idx = FilePaths.Num() - 1; Idx >= 0; --Idx)
		{
			if (IsIgnore(FilePaths[Idx]))
			{
				FilePaths.RemoveAtSwap(Idx);
			}
		}
	};

	StripIgnores(MissingFromPruned);
	StripIgnores(MissingFromFull);

	if (MissingFromPruned.Num() == 0 && MissingFromFull.Num() == 0)
	{
		return;
	}
	MissingFromPruned.Sort();
	MissingFromFull.Sort();

	// TODO: Restore this as an Error once we modify IPlatformFile::IterateDirectoryRecursively to declare its filefilter so we can ignore the spurious
	// discovered files that are not part of the fully filtered query
	UE_LOGF(LogPakFile, Error, "FindPrunedFilesAtPath('%ls') for PakFile '%ls' found a different list in the FullDirectory than in the PrunedDirectory. " "Change the calling code or add the files to Engine:[Pak]:WildcardsToKeepInPakStringIndex or Engine:[Pak]:IndexValidationIgnore.",
		InPath, *PakFilename);

#if !NO_LOGGING && !UE_BUILD_SHIPPING
	// Logging callstacks is expensive (multiple seconds long). Only do it the first time a path is seen, and only for the first
	// few paths.
	static TSet<FString> AlreadyLoggedCallstack;
	static FCriticalSection AlreadyLoggedCallstackLock;
	constexpr int32 CallstackLogDirsMax = 10;
	bool bShouldLogCallstack = false;
	if (AlreadyLoggedCallstack.Num() < CallstackLogDirsMax) // check to avoid taking critical section if unnecessary
	{
		FScopeLock AlreadyLoggedCallstackScopeLock(&AlreadyLoggedCallstackLock);
		if (AlreadyLoggedCallstack.Num() < CallstackLogDirsMax) // check again since other thread may have modified it
		{
			bool bAlreadyLogged;
			AlreadyLoggedCallstack.Add(FString(InPath), &bAlreadyLogged);
			bShouldLogCallstack = !bAlreadyLogged;
		}
	}
	if (bShouldLogCallstack)
	{
		UE_LOGF(LogPakFile, Warning, "Callstack of FindPrunedFilesAtPath('%ls'):", InPath);
		FDebug::DumpStackTraceToLog(ELogVerbosity::Warning);
	}
#endif

	if (MissingFromPruned.Num() > 0)
	{
		for (const FString& Missing : MissingFromPruned)
		{
			UE_LOGF(LogPakFile, Warning, "MissingPrunedPakFile: %ls", *Missing);
		}
	}
	if (MissingFromFull.Num() > 0)
	{
		UE_LOGF(LogPakFile, Error, "Some files in the PrunedDirectory are missing from the FullDirectory.  This is a logic error in FPakFile since the PrunedDirectory should be a subset of the FullDirectory.");
		for (const FString& Missing : MissingFromFull)
		{
			UE_LOGF(LogPakFile, Warning, "MissingFullPakFile: %ls", *Missing);
		}
	}
}
#endif

bool FPakFile::RecreatePakReaders(IPlatformFile* LowerLevel)
{
	FScopeLock ScopedLock(&ReadersCriticalSection);

	if (CurrentlyUsedReaders > 0)
	{
		UE_LOGF(LogPakFile, Error, "Recreating pak readers while we have readers loaned out, this may be lead to crashes or decryption problems.");
	}

	// need to reset the decryptor as it will hold a pointer to the first created pak reader
	Decryptor.Reset();

	TArray<FArchiveAndLastAccessTime> TempReaders;

	// Create a new PakReader *per* instance that was already mapped
	for (const FArchiveAndLastAccessTime& Reader : Readers)
	{
		TUniquePtr<FArchive> PakReader = TUniquePtr<FArchive>(CreatePakReader(LowerLevel, *GetFilename()));
		if (!PakReader)
		{
			UE_LOGF(LogPakFile, Warning, "Unable to re-create pak \"%ls\" handle", *GetFilename());
			return false;
		}
		TempReaders.Add(FArchiveAndLastAccessTime{ MoveTemp(PakReader), Reader.LastAccessTime });
	}

	// replace the current Readers with the newly created pak readers leaving them to out of scope
	Readers= MoveTemp(TempReaders);

	return true;
}

FSharedPakReader FPakFile::GetSharedReader(IPlatformFile* LowerLevel)
{
	LLM_SCOPE_BYTAG(PakSharedReaders);
	FArchive* PakReader = nullptr;
	{
		FScopeLock ScopedLock(&ReadersCriticalSection);
		if (Readers.Num())
		{
			FArchiveAndLastAccessTime Reader = Readers.Pop();
			PakReader = Reader.Archive.Release();
		}
		else
		{
			// Create a new FArchive reader and pass it to the new handle.
			PakReader = CreatePakReader(LowerLevel, *GetFilename());

			if (!PakReader)
			{
				UE_LOGF(LogPakFile, Warning, "Unable to create pak \"%ls\" handle", *GetFilename());
			}
		}
		++CurrentlyUsedReaders;
	}

	return FSharedPakReader(PakReader, this);
}

void FPakFile::ReturnSharedReader(FArchive* Archive)
{
	FScopeLock ScopedLock(&ReadersCriticalSection);
	--CurrentlyUsedReaders;
	Readers.Push(FArchiveAndLastAccessTime{ TUniquePtr<FArchive>{Archive }, FPlatformTime::Seconds()});
}

void FPakFile::ReleaseOldReaders(double MaxAgeSeconds)
{
	if (ReadersCriticalSection.TryLock())
	{
		ON_SCOPE_EXIT
		{
			ReadersCriticalSection.Unlock();
		};
		double SearchTime = FPlatformTime::Seconds() - MaxAgeSeconds;
		for (int32 i = Readers.Num() - 1; i >= 0; --i)
		{
			const FArchiveAndLastAccessTime& Reader = Readers[i];
			if (Reader.LastAccessTime <= SearchTime)
			{
				// Remove this and all readers older than it (pushed before it)
				Readers.RemoveAt(0, i + 1);
				break;
			}
		}

		if (Readers.Num() == 0 && CurrentlyUsedReaders == 0)
		{
			Decryptor.Reset();
		}
	}
}

const FPakEntryLocation* FPakFile::FindLocationFromIndex(const FString& FullPath, const FString& MountPoint, const FPathHashIndex& PathHashIndex, uint64 PathHashSeed, int32 PakFileVersion)
{
	const TCHAR* RelativePathFromMount = GetRelativeFilePathFromMountPointer(FullPath, MountPoint);
	if (!RelativePathFromMount)
	{
		return nullptr;
	}
	uint64 PathHash = HashPath(RelativePathFromMount, PathHashSeed, PakFileVersion);
	return PathHashIndex.Find(PathHash);
}

const FPakEntryLocation* FPakFile::FindLocationFromIndex(const FString& FullPath, const FString& MountPoint, const FDirectoryIndex& DirectoryIndex)
{
	if (!FullPath.StartsWith(MountPoint))
	{
		return nullptr;
	}
	FStringView RelativePathFromMount = FStringView(FullPath).Mid(MountPoint.Len());
	FStringView RelativeDirName(RelativePathFromMount);
	FStringView CleanFileName;
	if (RelativeDirName.IsEmpty())
	{
		return nullptr;
	}
	SplitPathInline(RelativeDirName, CleanFileName);
	const FPakDirectory* PakDirectory = DirectoryIndex.FindByHash(GetTypeHash(RelativeDirName), RelativeDirName);
	if (PakDirectory)
	{
		return PakDirectory->FindByHash(GetTypeHash(CleanFileName), FUtf8String(CleanFileName));
	}
	return nullptr;
}

const FPakEntryLocation* FPakFile::FindLocationFromIndex(const FString& FullPath,
	const FDirectoryIndex& InDirectoryIndex, const FDirectoryTreeIndex& InDirectoryTreeIndex) const
{
#if ENABLE_PAKFILE_USE_DIRECTORY_TREE
	if (ShouldUseDirectoryTree())
	{
		const FPakEntryLocation* Result = nullptr;
		if (FullPath.StartsWith(MountPoint))
		{
			FStringView RelativePathFromMount = FStringView(FullPath).Mid(MountPoint.Len());
			FStringView RelativeDirName(RelativePathFromMount);
			FStringView CleanFileName;
			if (!RelativeDirName.IsEmpty())
			{
				SplitPathInline(RelativeDirName, CleanFileName);
				const FPakDirectory* PakDirectory = InDirectoryTreeIndex.Find(RelativeDirName);
				if (PakDirectory)
				{
					Result = PakDirectory->FindByHash(GetTypeHash(CleanFileName), FUtf8String(CleanFileName));
				}
			}
		}

#if !UE_BUILD_SHIPPING
		if (GPak_ValidateDirectoryTreeSearchConsistency)
		{
			const FPakEntryLocation* IndexedResult = FindLocationFromIndex(FullPath, MountPoint, InDirectoryIndex);
			if ((Result != nullptr) != (IndexedResult != nullptr))
			{
				UE_LOGF(LogPakFile, Fatal, "Mismatch between directoryindex and directorytreeindex search when searching for [%ls] in pak [%ls]",
					*FullPath, *GetFilename());
			}
		}
#endif // !UE_BUILD_SHIPPING
		return Result;
	}
	else
#endif // ENABLE_PAKFILE_USE_DIRECTORY_TREE
	{
		return FindLocationFromIndex(FullPath, MountPoint, InDirectoryIndex);
	}
}

FPakFile::EFindResult FPakFile::Find(const FString& FullPath, FPakEntry* OutEntry) const
{
	//QUICK_SCOPE_CYCLE_COUNTER(PakFileFind);

	const FPakEntryLocation* PakEntryLocation;
#if ENABLE_PAKFILE_RUNTIME_PRUNING_VALIDATE
	if (IsPakValidatePruning() && bHasPathHashIndex && bHasFullDirectoryIndex)
	{
		const FPakEntryLocation* PathHashLocation = nullptr;
		PathHashLocation = FindLocationFromIndex(FullPath, MountPoint, PathHashIndex, PathHashSeed, Info.Version);

		const FPakEntryLocation* DirectoryLocation = nullptr;

		{
			FScopedPakDirectoryIndexAccess ScopeAccess(*this);
			DirectoryLocation = FindLocationFromIndex(FullPath, DirectoryIndex, DirectoryTreeIndex);
		}

		if ((PathHashLocation != nullptr) != (DirectoryLocation != nullptr))
		{
			const TCHAR* FoundName = TEXT("PathHashIndex");
			const TCHAR* NotFoundName = TEXT("FullDirectoryIndex");
			if (!PathHashLocation)
			{
				Swap(FoundName, NotFoundName);
			}
			UE_LOGF(LogPakFile, Error, "PathHashIndex does not match FullDirectoryIndex. Pakfile '%ls' has '%ls' in its %ls but not in its %ls.",
				*PakFilename, *FullPath, FoundName, NotFoundName);
		}
		PakEntryLocation = PathHashLocation ? PathHashLocation : DirectoryLocation;
	}
	else
#endif
	{
		if (bHasPathHashIndex)
		{
			PakEntryLocation = FindLocationFromIndex(FullPath, MountPoint, PathHashIndex, PathHashSeed, Info.Version);
		}
		else
		{
			// When we are using a pruned directory index, and no pathhash, calling Find with a filename will fail if the filename
			// was pruned. Therefore we don't support calling it in pakfiles with a pruned directory index. But it's okay to call on
			// an empty pakfile since that would return NotFound for every file.
			check(Files.IsEmpty() || bHasFullDirectoryIndex);
			FScopedPakDirectoryIndexAccess ScopeAccess(*this);
			PakEntryLocation = FindLocationFromIndex(FullPath, DirectoryIndex, DirectoryTreeIndex);
		}
	}
	if (!PakEntryLocation)
	{
		return EFindResult::NotFound;
	}

	return GetPakEntry(*PakEntryLocation, OutEntry);
}

void FPakFile::AddSpecialFile(const FPakEntry& Entry, const FString& Filename)
{
	MakeDirectoryFromPath(MountPoint);

	// TODO: This function is not threadsafe; readers of the Indexes will be invalidated when we modify them
	// To make it threadsafe would require always holding the lock around any read of either index, which is
	// more expensive than we want to support this debug feature
	FPakEntryLocation EntryLocation;
	if (!Entry.IsDeleteRecord())
	{
		// Add new file info.
		TArray<uint8> NewEncodedPakEntries;
		FMemoryWriter MemoryWriter(NewEncodedPakEntries);
		EntryLocation = FPakEntryLocation::CreateFromOffsetIntoEncoded(EncodedPakEntries.Num());
		if (EncodePakEntry(MemoryWriter, Entry, Info))
		{
			EncodedPakEntries.Append(NewEncodedPakEntries);
			EncodedPakEntries.Shrink();
		}
		else
		{
			EntryLocation = FPakEntryLocation::CreateFromListIndex(Files.Num());
			Files.Add(Entry);
			Files.Shrink();
		}
		NumEntries++;
	}

	FPathHashIndex* PathHashToWrite = bHasPathHashIndex ? &PathHashIndex : nullptr;
#if ENABLE_PAKFILE_USE_DIRECTORY_TREE
	if (ShouldUseDirectoryTree())
	{
		AddEntryToIndex(Filename, EntryLocation, MountPoint, PathHashSeed,
#if !UE_BUILD_SHIPPING
			GPak_ValidateDirectoryTreeSearchConsistency ? &DirectoryIndex : nullptr,
#else
			nullptr,
#endif // !UE_BUILD_SHIPPING
			&DirectoryTreeIndex,
			PathHashToWrite, nullptr /* CollisionDetection */, Info.Version);
	}
	else
#endif // ENABLE_PAKFILE_USE_DIRECTORY_TREE
	{
		AddEntryToIndex(Filename, EntryLocation, MountPoint, PathHashSeed, &DirectoryIndex, nullptr,
			PathHashToWrite, nullptr /* CollisionDetection */, Info.Version);
	}
}

FPakFile::FBaseIterator& FPakFile::FBaseIterator::operator++()
{
	switch (IteratorType)
	{
	case EIteratorType::PathHash:
		++GetPathHashIt();
		break;
	case EIteratorType::DirectoryTree:
		[[fallthrough]];
	case EIteratorType::DirectoryIndex:
		++GetFileIt();
		break;
	default:
		checkNoEntry();
		break;
	}
	AdvanceToValid();
	return *this;
}

FPakFile::FBaseIterator::operator bool() const
{
	switch (IteratorType)
	{
	case EIteratorType::PathHash:
		return (bool)GetPathHashIt();
	case EIteratorType::DirectoryTree:
#if ENABLE_PAKFILE_USE_DIRECTORY_TREE
		return (bool)GetTreeIt();
#else
		check(false);
		return false;
#endif
	case EIteratorType::DirectoryIndex:
		return (bool)GetDirectoryIndexIt();
	default:
		checkNoEntry();
		return false;
	}
}

bool FPakFile::FBaseIterator::operator !() const
{
	return !(bool)*this;
}

const FPakEntry& FPakFile::FBaseIterator::Info() const
{
	PakFile->GetPakEntry(GetPakEntryIndex(), &PakEntry);
	return PakEntry;
}

bool FPakFile::FBaseIterator::HasFilename() const
{
	return (IteratorType == EIteratorType::DirectoryIndex) | (IteratorType == EIteratorType::DirectoryTree);
}

FPakFile::FBaseIterator::FBaseIterator(const FPakFile& InPakFile, bool bInIncludeDeleted, bool bInUsePathHash)
	: PakFile(&InPakFile)
	, bIncludeDeleted(bInIncludeDeleted)
#if ENABLE_PAKFILE_RUNTIME_PRUNING
	, bRequiresDirectoryIndexLock(false)
#endif
{
	if (bInUsePathHash)
	{
		check(PakFile->bHasPathHashIndex);
		IteratorType = EIteratorType::PathHash;
		PathHashIt.Emplace(PakFile->PathHashIndex);
	}
	else
	{
#if ENABLE_PAKFILE_RUNTIME_PRUNING
		bRequiresDirectoryIndexLock = PakFile->RequiresDirectoryIndexLock();
		if (bRequiresDirectoryIndexLock)
		{
			PakFile->DirectoryIndexLock.ReadLock();
		}
#endif
#if ENABLE_PAKFILE_USE_DIRECTORY_TREE
		if (PakFile->ShouldUseDirectoryTree())
		{
			IteratorType = EIteratorType::DirectoryTree;
			TreeIt.Reset(new FDirectoryTreeIndex::FConstIterator(PakFile->DirectoryTreeIndex.CreateConstIterator()));
		}
		else
#endif
		{
			IteratorType = EIteratorType::DirectoryIndex;
			DirectoryIndexIt.Emplace(PakFile->DirectoryIndex);
		}
		if (IsDirectoryItValid())
		{
			FileIt.Emplace(GetDirectoryItValue());
		}
	}

	AdvanceToValid();
}

#if ENABLE_PAKFILE_RUNTIME_PRUNING
FPakFile::FBaseIterator::~FBaseIterator()
{
	if (bRequiresDirectoryIndexLock)
	{
		PakFile->DirectoryIndexLock.ReadUnlock();
	}
}
#endif

const FString& FPakFile::FBaseIterator::Filename() const
{
	if (IteratorType == EIteratorType::PathHash)
	{
		// Filenames are not supported, CachedFilename is always empty 
	}
	else
	{
		checkf((bool)*this, TEXT("It is not legal to call Filename() on an invalid iterator"));
		if (CachedFilename.IsEmpty())
		{
			CachedFilename = PakPathCombine(GetDirectoryItKey(), GetFileIt()->Key);
		}
	}
	return CachedFilename;
}

FPakEntryLocation FPakFile::FBaseIterator::GetPakEntryIndex() const
{
	switch (IteratorType)
	{
	case EIteratorType::PathHash:
		return GetPathHashIt().Value();
	case EIteratorType::DirectoryTree:
		[[fallthrough]];
	case EIteratorType::DirectoryIndex:
		return GetFileIt().Value();
	default:
		checkNoEntry();
		return FPakEntryLocation();
	}
}

void FPakFile::FBaseIterator::AdvanceToValid()
{
	if (IteratorType == EIteratorType::PathHash)
	{
		while (GetPathHashIt() && !bIncludeDeleted && Info().IsDeleteRecord())
		{
			++GetPathHashIt();
		}
	}
	else
	{
		check((IteratorType == EIteratorType::DirectoryTree) | (IteratorType == EIteratorType::DirectoryIndex));
		while (IsDirectoryItValid() && (!GetFileIt() || (!bIncludeDeleted && Info().IsDeleteRecord())))
		{
			if (GetFileIt())
			{
				++GetFileIt();
			}
			else
			{
				// No more files in the current directory, jump to the next one.
				IncrementDirectoryIt();
				if (IsDirectoryItValid())
				{
					FileIt.Emplace(GetDirectoryItValue());
				}
			}
		}
		CachedFilename.Reset();
	}
}

FPakFile::FPathHashIndex::TConstIterator& FPakFile::FBaseIterator::GetPathHashIt()
{
	check(IteratorType == EIteratorType::PathHash);
	return *PathHashIt;
}
const FPakFile::FPathHashIndex::TConstIterator& FPakFile::FBaseIterator::GetPathHashIt() const
{
	return const_cast<FBaseIterator*>(this)->GetPathHashIt();
}

bool FPakFile::FBaseIterator::IsDirectoryItValid() const
{
	check((IteratorType == EIteratorType::DirectoryIndex) | (IteratorType == EIteratorType::DirectoryTree));
#if ENABLE_PAKFILE_USE_DIRECTORY_TREE
	if (IteratorType == EIteratorType::DirectoryTree)
	{
		return (bool)GetTreeIt();
	}
	else
#endif
	{
		return (bool)GetDirectoryIndexIt();
	}
}

void FPakFile::FBaseIterator::IncrementDirectoryIt()
{
	check((IteratorType == EIteratorType::DirectoryIndex) | (IteratorType == EIteratorType::DirectoryTree));
#if ENABLE_PAKFILE_USE_DIRECTORY_TREE
	if (IteratorType == EIteratorType::DirectoryTree)
	{
		++GetTreeIt();
	}
	else
#endif
	{
		++GetDirectoryIndexIt();
	}
}

FStringView FPakFile::FBaseIterator::GetDirectoryItKey() const
{
	check((IteratorType == EIteratorType::DirectoryIndex) | (IteratorType == EIteratorType::DirectoryTree));
#if ENABLE_PAKFILE_USE_DIRECTORY_TREE
	if (IteratorType == EIteratorType::DirectoryTree)
	{
		return GetTreeIt()->Key;
	}
	else
#endif
	{
		return GetDirectoryIndexIt()->Key;
	}
}

const FPakDirectory& FPakFile::FBaseIterator::GetDirectoryItValue() const
{
	check((IteratorType == EIteratorType::DirectoryIndex) | (IteratorType == EIteratorType::DirectoryTree));
#if ENABLE_PAKFILE_USE_DIRECTORY_TREE
	if (IteratorType == EIteratorType::DirectoryTree)
	{
		return GetTreeIt()->Value;
	}
	else
#endif
	{
		return GetDirectoryIndexIt()->Value;
	}
}

FPakDirectory::TConstIterator& FPakFile::FBaseIterator::GetFileIt()
{
	check((IteratorType == EIteratorType::DirectoryIndex) | (IteratorType == EIteratorType::DirectoryTree));
	return *FileIt;
}

const FPakDirectory::TConstIterator& FPakFile::FBaseIterator::GetFileIt() const
{
	return const_cast<FBaseIterator*>(this)->GetFileIt();
}

FPakFile::FDirectoryIndex::TConstIterator& FPakFile::FBaseIterator::GetDirectoryIndexIt()
{
	check(IteratorType == EIteratorType::DirectoryIndex);
	return *DirectoryIndexIt;
}

const FPakFile::FDirectoryIndex::TConstIterator& FPakFile::FBaseIterator::GetDirectoryIndexIt() const
{
	return const_cast<FBaseIterator*>(this)->GetDirectoryIndexIt();
}

#if ENABLE_PAKFILE_USE_DIRECTORY_TREE
FPakFile::FDirectoryTreeIndex::FConstIterator& FPakFile::FBaseIterator::GetTreeIt()
{
	check(IteratorType == EIteratorType::DirectoryTree);
	return *TreeIt;
}

const FPakFile::FDirectoryTreeIndex::FConstIterator& FPakFile::FBaseIterator::GetTreeIt() const
{
	return const_cast<FBaseIterator*>(this)->GetTreeIt();
}
#endif
