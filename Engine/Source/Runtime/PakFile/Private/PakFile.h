// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPlatformFilePak.h"

#if ENABLE_PAKFILE_USE_DIRECTORY_TREE
#if !UE_BUILD_SHIPPING
extern bool GPak_ValidateDirectoryTreeSearchConsistency;
#endif // UE_BUILD_SHIPPING
extern bool GPak_UseDirectoryTreeForPakSearch;
#endif // ENABLE_PAKFILE_USE_DIRECTORY_TREE
