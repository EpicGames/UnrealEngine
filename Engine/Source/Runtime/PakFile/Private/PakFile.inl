// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
 * Search the given FDirectoryIndex for all files under the given Directory.  Helper for FindFilesAtPath, called separately on the DirectoryIndex or Pruned DirectoryIndex. Does not use
 * FScopedPakDirectoryIndexAccess internally; caller is responsible for calling from within a lock.
 * Returned paths are full paths (include the mount point)
 */
template <typename ShouldVisitFunc, class ContainerType>
void FPakFile::FindFilesAtPathInIndex(const FDirectoryIndex& TargetIndex, const FDirectoryTreeIndex& TargetTreeIndex, ContainerType& OutFiles,
	const FString& FullSearchPath, const FVisitFilter<ShouldVisitFunc>& VisitFilter) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FindFilesAtPathInIndex);

	FStringView RelSearchPath;
	if (FullSearchPath.StartsWith(MountPoint))
	{
		RelSearchPath = FStringView(FullSearchPath).RightChop(MountPoint.Len());
	}
	else
	{
		// Directory is unnormalized and might not end with /; MountPoint is guaranteed to end with /.
		// Act as if we were called with a normalized directory if adding slash makes it match MountPoint.
		if (FStringView(FullSearchPath).StartsWith(FStringView(MountPoint).LeftChop(1)))
		{
			RelSearchPath = FStringView();
		}
		else
		{
			// Early out; directory does not start with MountPoint and so will not match any of files in this pakfile.
			return;
		}
	}

	TArray<FString> DirectoriesInPak; // List of all unique directories at path

#if ENABLE_PAKFILE_USE_DIRECTORY_TREE
	if (ShouldUseDirectoryTree())
	{
		FindFilesAtPathInTreeIndexInternal(RelSearchPath, TargetTreeIndex, OutFiles, DirectoriesInPak, FullSearchPath, VisitFilter);
		
#if !UE_BUILD_SHIPPING
		if (GPak_ValidateDirectoryTreeSearchConsistency)
		{
			ContainerType OutFilesIndexed;
			TArray<FString> OutDirectoriesInPakIndexed;
			FindFilesAtPathInIndexInternal(RelSearchPath, TargetIndex, OutFilesIndexed, OutDirectoriesInPakIndexed, FullSearchPath, VisitFilter);
			if (!ValidateDirectoryTreeSearchConsistency(OutFiles, DirectoriesInPak, OutFilesIndexed, OutDirectoriesInPakIndexed))
			{
				UE_LOGF(LogPakFile, Fatal,
					"Mismatch between directoryindex and directorytreeindex search when searching for [%ls] in pak [%ls]",
					*FString(RelSearchPath), *GetFilename());
			}
		}
#endif	// !UE_BUILD_SHIPPING
	}
	else
#endif	// ENABLE_PAKFILE_USE_DIRECTORY_TREE
	{
		FindFilesAtPathInIndexInternal(RelSearchPath, TargetIndex, OutFiles, DirectoriesInPak, FullSearchPath, VisitFilter);
	}
	OutFiles.Append(MoveTemp(DirectoriesInPak));
}

#if ENABLE_PAKFILE_USE_DIRECTORY_TREE
template <typename ShouldVisitFunc, class ContainerType>
void FPakFile::FindFilesAtPathInTreeIndexInternal(FStringView RelSearchPath, const FDirectoryTreeIndex& TargetTreeIndex,
	ContainerType& OutFiles, TArray<FString>& OutDirectories, const FString& FullSearchPath,
	const FVisitFilter<ShouldVisitFunc>& VisitFilter) const
{
	if (RelSearchPath.IsEmpty())
	{
		for (TPair<FStringView, const FPakDirectory&> Pair : TargetTreeIndex)
		{
			FindFilesAtPathInPakDirectoryInternal(MountPoint, Pair.Key, Pair.Value, OutFiles, OutDirectories,
				FullSearchPath, VisitFilter);
		}
	}
	else
	{
		const FPakDirectory* PakDirectory = TargetTreeIndex.Find(RelSearchPath);
		if (PakDirectory)
		{
			FindFilesAtPathInPakDirectoryInternal(MountPoint, RelSearchPath, *PakDirectory, OutFiles, OutDirectories,
				FullSearchPath, VisitFilter);

			if (VisitFilter.bRecursive || VisitFilter.bIncludeDirectories)
			{
				// TODO: Add TDirectoryTree::CreatePathIterator so we can avoid the copy of children and the redundant
				// lookups of the children in the tree.
				TArray<FString> OutChildDirectories;
				TargetTreeIndex.TryGetChildren(RelSearchPath, OutChildDirectories,
					VisitFilter.bRecursive ? EDirectoryTreeGetFlags::Recursive : EDirectoryTreeGetFlags::None);
				for (const FString& ChildDirectoryPath : OutChildDirectories)
				{
					FString RelChildPath = PakPathCombine(RelSearchPath, ChildDirectoryPath);
					MakeDirectoryFromPath(RelChildPath);
					if (const FPakDirectory* PakDirectoryChild = TargetTreeIndex.Find(RelChildPath))
					{
						FindFilesAtPathInPakDirectoryInternal(MountPoint, RelChildPath,
							*PakDirectoryChild, OutFiles, OutDirectories, FullSearchPath, VisitFilter);
					}
				}
			}
		}
	}
}

#if !UE_BUILD_SHIPPING
template <class ContainerType>
bool FPakFile::ValidateDirectoryTreeSearchConsistency(const ContainerType& FilesTree, const TArray<FString>& DirectoriesInPakTree, const ContainerType& FilesIndexed, const TArray<FString>& DirectoriesInPakIndexed) const
{
	if (FilesTree.Num() == FilesIndexed.Num())
	{
		for (const FString& File : FilesTree)
		{
			if (!FilesIndexed.Contains(File))
			{
				return false;
			}
		}
	}
	else
	{
		return false;
	}

	if (DirectoriesInPakTree.Num() == DirectoriesInPakIndexed.Num())
	{
		for (const FString& Dir : DirectoriesInPakTree)
		{
			if (!DirectoriesInPakIndexed.Contains(Dir))
			{
				return false;
			}
		}
	}
	else
	{
		return false;
	}

	return true;
}
#endif // UE_BUILD_SHIPPING

#endif // ENABLE_PAKFILE_USE_DIRECTORY_TREE

template <typename ShouldVisitFunc, class ContainerType>
void FPakFile::FindFilesAtPathInIndexInternal(const FStringView& RelSearchPath, const FDirectoryIndex& TargetIndex,
	ContainerType& OutFiles, TArray<FString>& OutDirectories,
	const FString& FullSearchPath, const FVisitFilter<ShouldVisitFunc>& VisitFilter) const
{
	for (TMap<FString, FPakDirectory>::TConstIterator It(TargetIndex); It; ++It)
	{
		// Check if the file is under the specified path.
		if (FStringView(It.Key()).StartsWith(RelSearchPath))
		{
			FindFilesAtPathInPakDirectoryInternal(MountPoint, It.Key(), It.Value(), OutFiles, OutDirectories,
				FullSearchPath, VisitFilter);
		}
	}
}

template <typename ShouldVisitFunc, class ContainerType>
void FPakFile::FindFilesAtPathInPakDirectoryInternal(const FString& MountPoint, FStringView RelPathInIndex,
	const FPakDirectory& PakDirectory, ContainerType& OutFiles, TArray<FString>& OutDirectoriesInPak,
	const FString& FullSearchPath, const FVisitFilter<ShouldVisitFunc>& VisitFilter)
{
	FString FullPathInIndex = PakPathCombine(MountPoint, RelPathInIndex);
	if (VisitFilter.bRecursive == true)
	{
		// Add everything
		if (VisitFilter.bIncludeFiles)
		{
			TStringBuilder<1024> FilePathUnderDirectory;
			for (FPakDirectory::TConstIterator FileIt(PakDirectory); FileIt; ++FileIt)
			{
				FilePathUnderDirectory << FileIt.Key();
				if (VisitFilter.ShouldVisit(FilePathUnderDirectory.ToView()))
				{
					OutFiles.Add(PakPathCombine(FullPathInIndex, FilePathUnderDirectory.ToView()));
				}

				FilePathUnderDirectory.Reset();
			}
		}
		if (VisitFilter.bIncludeDirectories)
		{
			if (FullSearchPath != FullPathInIndex)
			{
				if (VisitFilter.ShouldVisit(FullPathInIndex))
				{
					OutDirectoriesInPak.Add(MoveTemp(FullPathInIndex));
				}
			}
		}
	}
	else
	{
		int32 SubDirIndex = FullPathInIndex.Len() > FullSearchPath.Len()
			? FullPathInIndex.Find(TEXT("/"), ESearchCase::CaseSensitive, ESearchDir::FromStart, FullSearchPath.Len() + 1)
			: INDEX_NONE;
		// Add files in the specified folder only.
		if (VisitFilter.bIncludeFiles && SubDirIndex == INDEX_NONE)
		{
			TStringBuilder<1024> FilePathUnderDirectory;
			for (FPakDirectory::TConstIterator FileIt(PakDirectory); FileIt; ++FileIt)
			{
				FilePathUnderDirectory << FileIt.Key();
				if (VisitFilter.ShouldVisit(FilePathUnderDirectory.ToView()))
				{
					OutFiles.Add(PakPathCombine(FullPathInIndex, FilePathUnderDirectory.ToView()));
				}

				FilePathUnderDirectory.Reset();
			}
		}
		// Add sub-folders in the specified folder only
		if (VisitFilter.bIncludeDirectories && SubDirIndex >= 0)
		{
			FString SubDirPath = FullPathInIndex.Left(SubDirIndex + 1);
			if (VisitFilter.ShouldVisit(SubDirPath))
			{
				OutDirectoriesInPak.AddUnique(MoveTemp(SubDirPath));
			}
		}
	}
}

template <typename ShouldVisitFunc>
FPakFile::FVisitFilter<ShouldVisitFunc>::FVisitFilter(const ShouldVisitFunc& InShouldVisit, bool bInIncludeFiles, bool bInIncludeDirectories, bool bInRecursive)
	: ShouldVisit(InShouldVisit)
	, bIncludeFiles(bInIncludeFiles)
	, bIncludeDirectories(bInIncludeDirectories)
	, bRecursive(bInRecursive)
{

}

template <typename ShouldVisitFunc, class ContainerType>
void FPakFile::FindPrunedFilesAtPathInternal(const TCHAR* InPath, ContainerType& OutFiles, const FVisitFilter<ShouldVisitFunc>& VisitFilter) const
{
	// Make sure all directory names end with '/'.
	FString FullSearchPath(InPath);
	MakeDirectoryFromPath(FullSearchPath);

	// Check the specified path is under the mount point of this pak file.
	// The reverse case (MountPoint StartsWith Directory) is needed to properly handle
	// pak files that are a subdirectory of the actual directory.
	if (!FullSearchPath.StartsWith(MountPoint) && !MountPoint.StartsWith(FullSearchPath))
	{
		return;
	}

	FScopedPakDirectoryIndexAccess ScopeAccess(*this);
#if ENABLE_PAKFILE_RUNTIME_PRUNING_VALIDATE
	if (ShouldValidatePrunedDirectory())
	{
		TSet<FString> FullFoundFiles, PrunedFoundFiles;
		FindFilesAtPathInIndex(DirectoryIndex, DirectoryTreeIndex, FullFoundFiles, FullSearchPath, VisitFilter);
		FindFilesAtPathInIndex(PrunedDirectoryIndex, PrunedDirectoryTreeIndex, PrunedFoundFiles, FullSearchPath, VisitFilter);
		ValidateDirectorySearch(FullFoundFiles, PrunedFoundFiles, InPath);

		for (const FString& FoundFile : FullFoundFiles)
		{
			OutFiles.Add(FoundFile);
		}
	}
	else
#endif
	{
		FindFilesAtPathInIndex(DirectoryIndex, DirectoryTreeIndex, OutFiles, FullSearchPath, VisitFilter);
	}
}
