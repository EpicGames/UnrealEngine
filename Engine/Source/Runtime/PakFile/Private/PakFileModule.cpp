// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAL/IPlatformFileModule.h"
#include "HAL/PlatformFileManager.h"
#include "IPlatformFilePak.h"
#include "Modules/ModuleManager.h"

/**
 * Module for the pak file
 */
class FPakFileModule : public IPlatformFileModule
{
public:
	virtual IPlatformFile* GetPlatformFile() override
	{
		check(Singleton.IsValid());
		return Singleton.Get();
	}

	virtual void StartupModule() override
	{
		Singleton = MakeUnique<FPakPlatformFile>();
		FModuleManager::LoadModuleChecked<IModuleInterface>(TEXT("RSA"));
	}

	virtual void ShutdownModule() override
	{
		// remove ourselves from the platform file chain (there can be late writes after the shutdown).
		if (Singleton.IsValid())
		{
			if (FPlatformFileManager::Get().FindPlatformFile(Singleton.Get()->GetName()))
			{
				FPlatformFileManager::Get().RemovePlatformFile(Singleton.Get());
			}
		}

		Singleton.Reset();
	}

	TUniquePtr<IPlatformFile> Singleton;
};

IMPLEMENT_MODULE(FPakFileModule, PakFile);
