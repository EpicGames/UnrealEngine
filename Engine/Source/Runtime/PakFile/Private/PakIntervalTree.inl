// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

enum
{
	IntervalTreeInvalidIndex = 0
};

typedef uint32 TIntervalTreeIndex; // this is the arg type of TSparseArray::operator[]

typedef uint64 FJoinedOffsetAndPakIndex;

static FORCEINLINE uint16 GetRequestPakIndexLow(FJoinedOffsetAndPakIndex Joined)
{
	return uint16((Joined >> 48) & 0xffff);
}

static FORCEINLINE int64 GetRequestOffset(FJoinedOffsetAndPakIndex Joined)
{
	return int64(Joined & 0xffffffffffffll);
}

static FORCEINLINE FJoinedOffsetAndPakIndex MakeJoinedRequest(uint16 PakIndex, int64 Offset)
{
	check(Offset >= 0);
	return (FJoinedOffsetAndPakIndex(PakIndex) << 48) | Offset;
}

static uint32 GNextSalt = 1;

// This is like TSparseArray, only a bit safer and I needed some restrictions on resizing.
template<class TItem>
class TIntervalTreeAllocator
{
	TArray<TItem> Items;
	TArray<int32> FreeItems; //@todo make this into a linked list through the existing items
	uint32 Salt;
	uint32 SaltMask;
public:
	TIntervalTreeAllocator()
	{
		check(GNextSalt < 4);
		Salt = (GNextSalt++) << 30;
		SaltMask = MAX_uint32 << 30;
		verify((Alloc() & ~SaltMask) == IntervalTreeInvalidIndex); // we want this to always have element zero so we can figure out an index from a pointer
	}
	inline TIntervalTreeIndex Alloc()
	{
		int32 Result;
		if (FreeItems.Num())
		{
			Result = FreeItems.Pop();
		}
		else
		{
			Result = Items.Num();
			Items.AddUninitialized();

		}
		new ((void*)&Items[Result]) TItem();
		return Result | Salt;;
	}
	void EnsureNoRealloc(int32 NeededNewNum)
	{
		if (FreeItems.Num() + Items.GetSlack() < NeededNewNum)
		{
			Items.Reserve(Items.Num() + NeededNewNum);
		}
	}
	FORCEINLINE TItem& Get(TIntervalTreeIndex InIndex)
	{
		TIntervalTreeIndex Index = InIndex & ~SaltMask;
		check((InIndex & SaltMask) == Salt && Index != IntervalTreeInvalidIndex && Index >= 0 && Index < (uint32)Items.Num()); //&& !FreeItems.Contains(Index));
		return Items[Index];
	}
	FORCEINLINE void Free(TIntervalTreeIndex InIndex)
	{
		TIntervalTreeIndex Index = InIndex & ~SaltMask;
		check((InIndex & SaltMask) == Salt && Index != IntervalTreeInvalidIndex && Index >= 0 && Index < (uint32)Items.Num()); //&& !FreeItems.Contains(Index));
		Items[Index].~TItem();
		FreeItems.Push(Index);
		if (FreeItems.Num() + 1 == Items.Num())
		{
			// get rid everything to restore memory coherence
			Items.Empty();
			FreeItems.Empty();
			verify((Alloc() & ~SaltMask) == IntervalTreeInvalidIndex); // we want this to always have element zero so we can figure out an index from a pointer
		}
	}
	FORCEINLINE void CheckIndex(TIntervalTreeIndex InIndex)
	{
		TIntervalTreeIndex Index = InIndex & ~SaltMask;
		check((InIndex & SaltMask) == Salt && Index != IntervalTreeInvalidIndex && Index >= 0 && Index < (uint32)Items.Num()); // && !FreeItems.Contains(Index));
	}
};

class FIntervalTreeNode
{
public:
	TIntervalTreeIndex LeftChildOrRootOfLeftList;
	TIntervalTreeIndex RootOfOnList;
	TIntervalTreeIndex RightChildOrRootOfRightList;

	FIntervalTreeNode()
		: LeftChildOrRootOfLeftList(IntervalTreeInvalidIndex)
		, RootOfOnList(IntervalTreeInvalidIndex)
		, RightChildOrRootOfRightList(IntervalTreeInvalidIndex)
	{
	}
	~FIntervalTreeNode()
	{
		check(LeftChildOrRootOfLeftList == IntervalTreeInvalidIndex && RootOfOnList == IntervalTreeInvalidIndex && RightChildOrRootOfRightList == IntervalTreeInvalidIndex); // this routine does not handle recursive destruction
	}
};

static TIntervalTreeAllocator<FIntervalTreeNode> GIntervalTreeNodeNodeAllocator;

static FORCEINLINE uint64 HighBit(uint64 x)
{
	return x & (1ull << 63);
}

static FORCEINLINE bool IntervalsIntersect(uint64 Min1, uint64 Max1, uint64 Min2, uint64 Max2)
{
	return !(Max2 < Min1 || Max1 < Min2);
}

template<typename TItem>
// this routine assume that the pointers remain valid even though we are reallocating
static void AddToIntervalTree_Dangerous(
	TIntervalTreeIndex* RootNode,
	TIntervalTreeAllocator<TItem>& Allocator,
	TIntervalTreeIndex Index,
	uint64 MinInterval,
	uint64 MaxInterval,
	uint32 CurrentShift,
	uint32 MaxShift
)
{
	while (true)
	{
		if (*RootNode == IntervalTreeInvalidIndex)
		{
			*RootNode = GIntervalTreeNodeNodeAllocator.Alloc();
		}

		int64 MinShifted = HighBit(MinInterval << CurrentShift);
		int64 MaxShifted = HighBit(MaxInterval << CurrentShift);
		FIntervalTreeNode& Root = GIntervalTreeNodeNodeAllocator.Get(*RootNode);

		if (MinShifted == MaxShifted && CurrentShift < MaxShift)
		{
			CurrentShift++;
			RootNode = (!MinShifted) ? &Root.LeftChildOrRootOfLeftList : &Root.RightChildOrRootOfRightList;
		}
		else
		{
			TItem& Item = Allocator.Get(Index);
			if (MinShifted != MaxShifted) // crosses middle
			{
				Item.Next = Root.RootOfOnList;
				Root.RootOfOnList = Index;
			}
			else // we are at the leaf
			{
				if (!MinShifted)
				{
					Item.Next = Root.LeftChildOrRootOfLeftList;
					Root.LeftChildOrRootOfLeftList = Index;
				}
				else
				{
					Item.Next = Root.RightChildOrRootOfRightList;
					Root.RightChildOrRootOfRightList = Index;
				}
			}
			return;
		}
	}
}

template<typename TItem>
static void AddToIntervalTree(
	TIntervalTreeIndex* RootNode,
	TIntervalTreeAllocator<TItem>& Allocator,
	TIntervalTreeIndex Index,
	uint32 StartShift,
	uint32 MaxShift
)
{
	GIntervalTreeNodeNodeAllocator.EnsureNoRealloc(1 + MaxShift - StartShift);
	TItem& Item = Allocator.Get(Index);
	check(Item.Next == IntervalTreeInvalidIndex);
	uint64 MinInterval = GetRequestOffset(Item.OffsetAndPakIndex);
	uint64 MaxInterval = MinInterval + Item.Size - 1;
	AddToIntervalTree_Dangerous(RootNode, Allocator, Index, MinInterval, MaxInterval, StartShift, MaxShift);

}

template<typename TItem>
static FORCEINLINE bool ScanNodeListForRemoval(
	TIntervalTreeIndex* Iter,
	TIntervalTreeAllocator<TItem>& Allocator,
	TIntervalTreeIndex Index,
	uint64 MinInterval,
	uint64 MaxInterval
)
{
	while (*Iter != IntervalTreeInvalidIndex)
	{

		TItem& Item = Allocator.Get(*Iter);
		if (*Iter == Index)
		{
			*Iter = Item.Next;
			Item.Next = IntervalTreeInvalidIndex;
			return true;
		}
		Iter = &Item.Next;
	}
	return false;
}

template<typename TItem>
static bool RemoveFromIntervalTree(
	TIntervalTreeIndex* RootNode,
	TIntervalTreeAllocator<TItem>& Allocator,
	TIntervalTreeIndex Index,
	uint64 MinInterval,
	uint64 MaxInterval,
	uint32 CurrentShift,
	uint32 MaxShift
)
{
	bool bResult = false;
	if (*RootNode != IntervalTreeInvalidIndex)
	{
		int64 MinShifted = HighBit(MinInterval << CurrentShift);
		int64 MaxShifted = HighBit(MaxInterval << CurrentShift);
		FIntervalTreeNode& Root = GIntervalTreeNodeNodeAllocator.Get(*RootNode);

		if (!MinShifted && !MaxShifted)
		{
			if (CurrentShift == MaxShift)
			{
				bResult = ScanNodeListForRemoval(&Root.LeftChildOrRootOfLeftList, Allocator, Index, MinInterval, MaxInterval);
			}
			else
			{
				bResult = RemoveFromIntervalTree(&Root.LeftChildOrRootOfLeftList, Allocator, Index, MinInterval, MaxInterval, CurrentShift + 1, MaxShift);
			}
		}
		else if (!MinShifted && MaxShifted)
		{
			bResult = ScanNodeListForRemoval(&Root.RootOfOnList, Allocator, Index, MinInterval, MaxInterval);
		}
		else
		{
			if (CurrentShift == MaxShift)
			{
				bResult = ScanNodeListForRemoval(&Root.RightChildOrRootOfRightList, Allocator, Index, MinInterval, MaxInterval);
			}
			else
			{
				bResult = RemoveFromIntervalTree(&Root.RightChildOrRootOfRightList, Allocator, Index, MinInterval, MaxInterval, CurrentShift + 1, MaxShift);
			}
		}
		if (bResult)
		{
			if (Root.LeftChildOrRootOfLeftList == IntervalTreeInvalidIndex && Root.RootOfOnList == IntervalTreeInvalidIndex && Root.RightChildOrRootOfRightList == IntervalTreeInvalidIndex)
			{
				check(&Root == &GIntervalTreeNodeNodeAllocator.Get(*RootNode));
				GIntervalTreeNodeNodeAllocator.Free(*RootNode);
				*RootNode = IntervalTreeInvalidIndex;
			}
		}
	}
	return bResult;
}

template<typename TItem>
static bool RemoveFromIntervalTree(
	TIntervalTreeIndex* RootNode,
	TIntervalTreeAllocator<TItem>& Allocator,
	TIntervalTreeIndex Index,
	uint32 StartShift,
	uint32 MaxShift
)
{
	TItem& Item = Allocator.Get(Index);
	uint64 MinInterval = GetRequestOffset(Item.OffsetAndPakIndex);
	uint64 MaxInterval = MinInterval + Item.Size - 1;
	return RemoveFromIntervalTree(RootNode, Allocator, Index, MinInterval, MaxInterval, StartShift, MaxShift);
}

template<typename TItem>
static FORCEINLINE void ScanNodeListForRemovalFunc(
	TIntervalTreeIndex* Iter,
	TIntervalTreeAllocator<TItem>& Allocator,
	uint64 MinInterval,
	uint64 MaxInterval,
	TFunctionRef<bool(TIntervalTreeIndex)> Func
)
{
	while (*Iter != IntervalTreeInvalidIndex)
	{
		TItem& Item = Allocator.Get(*Iter);
		uint64 Offset = uint64(GetRequestOffset(Item.OffsetAndPakIndex));
		uint64 LastByte = Offset + uint64(Item.Size) - 1;

		// save the value and then clear it.
		TIntervalTreeIndex NextIndex = Item.Next;
		if (IntervalsIntersect(MinInterval, MaxInterval, Offset, LastByte) && Func(*Iter))
		{
			*Iter = NextIndex; // this may have already be deleted, so cannot rely on the memory block
		}
		else
		{
			Iter = &Item.Next;
		}
	}
}

template<typename TItem>
static void MaybeRemoveOverlappingNodesInIntervalTree(
	TIntervalTreeIndex* RootNode,
	TIntervalTreeAllocator<TItem>& Allocator,
	uint64 MinInterval,
	uint64 MaxInterval,
	uint64 MinNode,
	uint64 MaxNode,
	uint32 CurrentShift,
	uint32 MaxShift,
	TFunctionRef<bool(TIntervalTreeIndex)> Func
)
{
	if (*RootNode != IntervalTreeInvalidIndex)
	{
		int64 MinShifted = HighBit(MinInterval << CurrentShift);
		int64 MaxShifted = HighBit(MaxInterval << CurrentShift);
		FIntervalTreeNode& Root = GIntervalTreeNodeNodeAllocator.Get(*RootNode);
		uint64 Center = (MinNode + MaxNode + 1) >> 1;

		//UE_LOGF(LogTemp, Warning, "Exploring Node %X [%d, %d] %d%d     interval %llX %llX    node interval %llX %llX   center %llX  ", *RootNode, CurrentShift, MaxShift, !!MinShifted, !!MaxShifted, MinInterval, MaxInterval, MinNode, MaxNode, Center);


		if (!MinShifted)
		{
			if (CurrentShift == MaxShift)
			{
				//UE_LOGF(LogTemp, Warning, "LeftBottom %X [%d, %d] %d%d", *RootNode, CurrentShift, MaxShift, !!MinShifted, !!MaxShifted);
				ScanNodeListForRemovalFunc(&Root.LeftChildOrRootOfLeftList, Allocator, MinInterval, MaxInterval, Func);
			}
			else
			{
				//UE_LOGF(LogTemp, Warning, "LeftRecur %X [%d, %d] %d%d", *RootNode, CurrentShift, MaxShift, !!MinShifted, !!MaxShifted);
				MaybeRemoveOverlappingNodesInIntervalTree(&Root.LeftChildOrRootOfLeftList, Allocator, MinInterval, FMath::Min(MaxInterval, Center - 1), MinNode, Center - 1, CurrentShift + 1, MaxShift, Func);
			}
		}

		//UE_LOGF(LogTemp, Warning, "Center %X [%d, %d] %d%d", *RootNode, CurrentShift, MaxShift, !!MinShifted, !!MaxShifted);
		ScanNodeListForRemovalFunc(&Root.RootOfOnList, Allocator, MinInterval, MaxInterval, Func);

		if (MaxShifted)
		{
			if (CurrentShift == MaxShift)
			{
				//UE_LOGF(LogTemp, Warning, "RightBottom %X [%d, %d] %d%d", *RootNode, CurrentShift, MaxShift, !!MinShifted, !!MaxShifted);
				ScanNodeListForRemovalFunc(&Root.RightChildOrRootOfRightList, Allocator, MinInterval, MaxInterval, Func);
			}
			else
			{
				//UE_LOGF(LogTemp, Warning, "RightRecur %X [%d, %d] %d%d", *RootNode, CurrentShift, MaxShift, !!MinShifted, !!MaxShifted);
				MaybeRemoveOverlappingNodesInIntervalTree(&Root.RightChildOrRootOfRightList, Allocator, FMath::Max(MinInterval, Center), MaxInterval, Center, MaxNode, CurrentShift + 1, MaxShift, Func);
			}
		}

		//UE_LOGF(LogTemp, Warning, "Done Exploring Node %X [%d, %d] %d%d     interval %llX %llX    node interval %llX %llX   center %llX  ", *RootNode, CurrentShift, MaxShift, !!MinShifted, !!MaxShifted, MinInterval, MaxInterval, MinNode, MaxNode, Center);

		if (Root.LeftChildOrRootOfLeftList == IntervalTreeInvalidIndex && Root.RootOfOnList == IntervalTreeInvalidIndex && Root.RightChildOrRootOfRightList == IntervalTreeInvalidIndex)
		{
			check(&Root == &GIntervalTreeNodeNodeAllocator.Get(*RootNode));
			GIntervalTreeNodeNodeAllocator.Free(*RootNode);
			*RootNode = IntervalTreeInvalidIndex;
		}
	}
}


template<typename TItem>
static FORCEINLINE bool ScanNodeList(
	TIntervalTreeIndex Iter,
	TIntervalTreeAllocator<TItem>& Allocator,
	uint64 MinInterval,
	uint64 MaxInterval,
	TFunctionRef<bool(TIntervalTreeIndex)> Func
)
{
	while (Iter != IntervalTreeInvalidIndex)
	{
		TItem& Item = Allocator.Get(Iter);
		uint64 Offset = uint64(GetRequestOffset(Item.OffsetAndPakIndex));
		uint64 LastByte = Offset + uint64(Item.Size) - 1;
		if (IntervalsIntersect(MinInterval, MaxInterval, Offset, LastByte))
		{
			if (!Func(Iter))
			{
				return false;
			}
		}
		Iter = Item.Next;
	}
	return true;
}

template<typename TItem>
static bool OverlappingNodesInIntervalTree(
	TIntervalTreeIndex RootNode,
	TIntervalTreeAllocator<TItem>& Allocator,
	uint64 MinInterval,
	uint64 MaxInterval,
	uint64 MinNode,
	uint64 MaxNode,
	uint32 CurrentShift,
	uint32 MaxShift,
	TFunctionRef<bool(TIntervalTreeIndex)> Func
)
{
	if (RootNode != IntervalTreeInvalidIndex)
	{
		int64 MinShifted = HighBit(MinInterval << CurrentShift);
		int64 MaxShifted = HighBit(MaxInterval << CurrentShift);
		FIntervalTreeNode& Root = GIntervalTreeNodeNodeAllocator.Get(RootNode);
		uint64 Center = (MinNode + MaxNode + 1) >> 1;

		if (!MinShifted)
		{
			if (CurrentShift == MaxShift)
			{
				if (!ScanNodeList(Root.LeftChildOrRootOfLeftList, Allocator, MinInterval, MaxInterval, Func))
				{
					return false;
				}
			}
			else
			{
				if (!OverlappingNodesInIntervalTree(Root.LeftChildOrRootOfLeftList, Allocator, MinInterval, FMath::Min(MaxInterval, Center - 1), MinNode, Center - 1, CurrentShift + 1, MaxShift, Func))
				{
					return false;
				}
			}
		}
		if (!ScanNodeList(Root.RootOfOnList, Allocator, MinInterval, MaxInterval, Func))
		{
			return false;
		}
		if (MaxShifted)
		{
			if (CurrentShift == MaxShift)
			{
				if (!ScanNodeList(Root.RightChildOrRootOfRightList, Allocator, MinInterval, MaxInterval, Func))
				{
					return false;
				}
			}
			else
			{
				if (!OverlappingNodesInIntervalTree(Root.RightChildOrRootOfRightList, Allocator, FMath::Max(MinInterval, Center), MaxInterval, Center, MaxNode, CurrentShift + 1, MaxShift, Func))
				{
					return false;
				}
			}
		}
	}
	return true;
}

template<typename TItem>
static bool ScanNodeListWithShrinkingInterval(
	TIntervalTreeIndex Iter,
	TIntervalTreeAllocator<TItem>& Allocator,
	uint64 MinInterval,
	uint64& MaxInterval,
	TFunctionRef<bool(TIntervalTreeIndex)> Func
)
{
	while (Iter != IntervalTreeInvalidIndex)
	{
		TItem& Item = Allocator.Get(Iter);
		uint64 Offset = uint64(GetRequestOffset(Item.OffsetAndPakIndex));
		uint64 LastByte = Offset + uint64(Item.Size) - 1;
		//UE_LOGF(LogTemp, Warning, "Test Overlap %llu %llu %llu %llu", MinInterval, MaxInterval, Offset, LastByte);
		if (IntervalsIntersect(MinInterval, MaxInterval, Offset, LastByte))
		{
			//UE_LOGF(LogTemp, Warning, "Overlap %llu %llu %llu %llu", MinInterval, MaxInterval, Offset, LastByte);
			if (!Func(Iter))
			{
				return false;
			}
		}
		Iter = Item.Next;
	}
	return true;
}

template<typename TItem>
static bool OverlappingNodesInIntervalTreeWithShrinkingInterval(
	TIntervalTreeIndex RootNode,
	TIntervalTreeAllocator<TItem>& Allocator,
	uint64 MinInterval,
	uint64& MaxInterval,
	uint64 MinNode,
	uint64 MaxNode,
	uint32 CurrentShift,
	uint32 MaxShift,
	TFunctionRef<bool(TIntervalTreeIndex)> Func
)
{
	if (RootNode != IntervalTreeInvalidIndex)
	{

		int64 MinShifted = HighBit(MinInterval << CurrentShift);
		int64 MaxShifted = HighBit(FMath::Min(MaxInterval, MaxNode) << CurrentShift); // since MaxInterval is changing, we cannot clamp it during recursion.
		FIntervalTreeNode& Root = GIntervalTreeNodeNodeAllocator.Get(RootNode);
		uint64 Center = (MinNode + MaxNode + 1) >> 1;

		if (!MinShifted)
		{
			if (CurrentShift == MaxShift)
			{
				if (!ScanNodeListWithShrinkingInterval(Root.LeftChildOrRootOfLeftList, Allocator, MinInterval, MaxInterval, Func))
				{
					return false;
				}
			}
			else
			{
				if (!OverlappingNodesInIntervalTreeWithShrinkingInterval(Root.LeftChildOrRootOfLeftList, Allocator, MinInterval, MaxInterval, MinNode, Center - 1, CurrentShift + 1, MaxShift, Func)) // since MaxInterval is changing, we cannot clamp it during recursion.
				{
					return false;
				}
			}
		}
		if (!ScanNodeListWithShrinkingInterval(Root.RootOfOnList, Allocator, MinInterval, MaxInterval, Func))
		{
			return false;
		}
		MaxShifted = HighBit(FMath::Min(MaxInterval, MaxNode) << CurrentShift); // since MaxInterval is changing, we cannot clamp it during recursion.
		if (MaxShifted)
		{
			if (CurrentShift == MaxShift)
			{
				if (!ScanNodeListWithShrinkingInterval(Root.RightChildOrRootOfRightList, Allocator, MinInterval, MaxInterval, Func))
				{
					return false;
				}
			}
			else
			{
				if (!OverlappingNodesInIntervalTreeWithShrinkingInterval(Root.RightChildOrRootOfRightList, Allocator, FMath::Max(MinInterval, Center), MaxInterval, Center, MaxNode, CurrentShift + 1, MaxShift, Func))
				{
					return false;
				}
			}
		}
	}
	return true;
}


template<typename TItem>
static void MaskInterval(
	TIntervalTreeIndex Index,
	TIntervalTreeAllocator<TItem>& Allocator,
	uint64 MinInterval,
	uint64 MaxInterval,
	uint32 BytesToBitsShift,
	uint64* Bits
)
{
	TItem& Item = Allocator.Get(Index);
	uint64 Offset = uint64(GetRequestOffset(Item.OffsetAndPakIndex));
	uint64 LastByte = Offset + uint64(Item.Size) - 1;
	uint64 InterMinInterval = FMath::Max(MinInterval, Offset);
	uint64 InterMaxInterval = FMath::Min(MaxInterval, LastByte);
	if (InterMinInterval <= InterMaxInterval)
	{
		uint32 FirstBit = uint32((InterMinInterval - MinInterval) >> BytesToBitsShift);
		uint32 LastBit = uint32((InterMaxInterval - MinInterval) >> BytesToBitsShift);
		uint32 FirstQWord = FirstBit >> 6;
		uint32 LastQWord = LastBit >> 6;
		uint32 FirstBitQWord = FirstBit & 63;
		uint32 LastBitQWord = LastBit & 63;
		if (FirstQWord == LastQWord)
		{
			Bits[FirstQWord] |= ((MAX_uint64 << FirstBitQWord) & (MAX_uint64 >> (63 - LastBitQWord)));
		}
		else
		{
			Bits[FirstQWord] |= (MAX_uint64 << FirstBitQWord);
			for (uint32 QWordIndex = FirstQWord + 1; QWordIndex < LastQWord; QWordIndex++)
			{
				Bits[QWordIndex] = MAX_uint64;
			}
			Bits[LastQWord] |= (MAX_uint64 >> (63 - LastBitQWord));
		}
	}
}



template<typename TItem>
static void OverlappingNodesInIntervalTreeMask(
	TIntervalTreeIndex RootNode,
	TIntervalTreeAllocator<TItem>& Allocator,
	uint64 MinInterval,
	uint64 MaxInterval,
	uint64 MinNode,
	uint64 MaxNode,
	uint32 CurrentShift,
	uint32 MaxShift,
	uint32 BytesToBitsShift,
	uint64* Bits
)
{
	OverlappingNodesInIntervalTree(
		RootNode,
		Allocator,
		MinInterval,
		MaxInterval,
		MinNode,
		MaxNode,
		CurrentShift,
		MaxShift,
		[&Allocator, MinInterval, MaxInterval, BytesToBitsShift, Bits](TIntervalTreeIndex Index) -> bool
	{
		MaskInterval(Index, Allocator, MinInterval, MaxInterval, BytesToBitsShift, Bits);
		return true;
	}
	);
}
