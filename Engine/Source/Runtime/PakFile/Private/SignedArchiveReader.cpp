// Copyright Epic Games, Inc. All Rights Reserved.

#include "SignedArchiveReader.h"
#include "HAL/FileManager.h"
#include "HAL/Event.h"
#include "HAL/RunnableThread.h"

DECLARE_FLOAT_ACCUMULATOR_STAT(TEXT("FChunkCacheWorker.ProcessQueue"), STAT_FChunkCacheWorker_ProcessQueue, STATGROUP_PakFile);
DECLARE_FLOAT_ACCUMULATOR_STAT(TEXT("FChunkCacheWorker.CheckSignature"), STAT_FChunkCacheWorker_CheckSignature, STATGROUP_PakFile);
DECLARE_FLOAT_ACCUMULATOR_STAT(TEXT("FChunkCacheWorker.RequestQueueUpdate"), STAT_FChunkCacheWorker_RequestQueueUpdate, STATGROUP_PakFile);
DECLARE_FLOAT_ACCUMULATOR_STAT(TEXT("FChunkCacheWorker.RequestWaitTime"), STAT_FChunkCacheWorker_RequestWaitTime, STATGROUP_PakFile);

DECLARE_FLOAT_ACCUMULATOR_STAT(TEXT("FChunkCacheWorker.Serialize"), STAT_FChunkCacheWorker_Serialize, STATGROUP_PakFile);
DECLARE_FLOAT_ACCUMULATOR_STAT(TEXT("FChunkCacheWorker.HashBuffer"), STAT_FChunkCacheWorker_HashBuffer, STATGROUP_PakFile);
DECLARE_FLOAT_ACCUMULATOR_STAT(TEXT("FChunkCacheWorker.WaitingForEvent"), STAT_FChunkCacheWorker_WaitingForEvent, STATGROUP_PakFile);
DECLARE_FLOAT_ACCUMULATOR_STAT(TEXT("FChunkCacheWorker.GetFreeBuffer"), STAT_FChunkCacheWorker_GetFreeBuffer, STATGROUP_PakFile);
DECLARE_FLOAT_ACCUMULATOR_STAT(TEXT("FChunkCacheWorker.ReleaseBuffer"), STAT_FChunkCacheWorker_ReleaseBuffer, STATGROUP_PakFile);

DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("FChunkCacheWorker.NumProcessQueues"), STAT_FChunkCacheWorker_NumProcessQueue, STATGROUP_PakFile);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("FChunkCacheWorker.NumProcessQueuesWithWork"), STAT_FChunkCacheWorker_NumProcessQueueWithWork, STATGROUP_PakFile);

DECLARE_FLOAT_ACCUMULATOR_STAT(TEXT("FSignedArchiveReader.Serialize"), STAT_SignedArchiveReader_Serialize, STATGROUP_PakFile);
DECLARE_FLOAT_ACCUMULATOR_STAT(TEXT("FSignedArchiveReader.PreCacheChunks"), STAT_SignedArchiveReader_PreCacheChunks, STATGROUP_PakFile);
DECLARE_FLOAT_ACCUMULATOR_STAT(TEXT("FSignedArchiveReader.CopyFromNewCache"), STAT_SignedArchiveReader_CopyFromNewCache, STATGROUP_PakFile);
DECLARE_FLOAT_ACCUMULATOR_STAT(TEXT("FSignedArchiveReader.CopyFromExistingCache"), STAT_SignedArchiveReader_CopyFromExistingCache, STATGROUP_PakFile);
DECLARE_FLOAT_ACCUMULATOR_STAT(TEXT("FSignedArchiveReader.ProcessChunkRequests"), STAT_SignedArchiveReader_ProcessChunkRequests, STATGROUP_PakFile);
DECLARE_FLOAT_ACCUMULATOR_STAT(TEXT("FSignedArchiveReader.WaitingForChunkWorker"), STAT_SignedArchiveReader_WaitForChunkWorker, STATGROUP_PakFile);

DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("FSignedArchiveReader.NumSerializes"), STAT_SignedArchiveReader_NumSerializes, STATGROUP_PakFile);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("FSignedArchiveReader.NumChunkRequests"), STAT_SignedArchiveReader_NumChunkRequests, STATGROUP_PakFile);


FChunkCacheWorker::FChunkCacheWorker(TUniquePtr<FArchive> InReader, const TCHAR* Filename)
	: Thread(nullptr)
	, Reader(MoveTemp(InReader))
	, QueuedRequestsEvent(nullptr)
{
	Signatures = FPakPlatformFile::GetPakSignatureFile(Filename);

	if (Signatures.IsValid())
	{
		const bool bEnableMultithreading = FPlatformProcess::SupportsMultithreading();
		if (bEnableMultithreading)
		{
			QueuedRequestsEvent = FPlatformProcess::GetSynchEventFromPool();
			Thread = FRunnableThread::Create(this, TEXT("FChunkCacheWorker"), 0, TPri_BelowNormal);
		}
	}
}

FChunkCacheWorker::~FChunkCacheWorker()
{
	delete Thread;
	Thread = nullptr;
	if (QueuedRequestsEvent != nullptr)
	{
		FPlatformProcess::ReturnSynchEventToPool(QueuedRequestsEvent);
		QueuedRequestsEvent = nullptr;
	}
}

bool FChunkCacheWorker::Init()
{
	return true;
}

uint32 FChunkCacheWorker::Run()
{	
	check(QueuedRequestsEvent);
	while (StopTaskCounter.GetValue() == 0)
	{
		if (QueuedRequestsEvent->Wait())
		{
			ProcessQueue();
		}
	}
	return 0;
}

void FChunkCacheWorker::Stop()
{
	StopTaskCounter.Increment();
	if (QueuedRequestsEvent)
	{
		QueuedRequestsEvent->Trigger();
	}
}

FChunkBuffer* FChunkCacheWorker::GetCachedChunkBuffer(int32 ChunkIndex)
{
	for (int32 BufferIndex = 0; BufferIndex < MaxCachedChunks; ++BufferIndex)
	{
		if (CachedChunks[BufferIndex].ChunkIndex == ChunkIndex)
		{
			// Update access info and lock
			CachedChunks[BufferIndex].LockCount++;
			CachedChunks[BufferIndex].LastAccessTime = FPlatformTime::Seconds();
			return &CachedChunks[BufferIndex];
		}
	}
	return NULL;
}

FChunkBuffer* FChunkCacheWorker::GetFreeBuffer()
{
	SCOPE_SECONDS_ACCUMULATOR(STAT_FChunkCacheWorker_GetFreeBuffer);

	// Find the least recently accessed, free buffer.
	FChunkBuffer* LeastRecentFreeBuffer = NULL;
	for (int32 BufferIndex = 0; BufferIndex < MaxCachedChunks; ++BufferIndex)
	{
		if (CachedChunks[BufferIndex].LockCount == 0 && 
			 (LeastRecentFreeBuffer == NULL || LeastRecentFreeBuffer->LastAccessTime > CachedChunks[BufferIndex].LastAccessTime))
		{
			LeastRecentFreeBuffer = &CachedChunks[BufferIndex];
		}
	}
	if (LeastRecentFreeBuffer)
	{
		// Update access info and lock
		LeastRecentFreeBuffer->LockCount++;
		LeastRecentFreeBuffer->LastAccessTime = FPlatformTime::Seconds();
	}
	return LeastRecentFreeBuffer;
}

void FChunkCacheWorker::ReleaseBuffer(int32 ChunkIndex)
{
	SCOPE_SECONDS_ACCUMULATOR(STAT_FChunkCacheWorker_ReleaseBuffer);

	for (int32 BufferIndex = 0; BufferIndex < MaxCachedChunks; ++BufferIndex)
	{
		if (CachedChunks[BufferIndex].ChunkIndex == ChunkIndex)
		{
			CachedChunks[BufferIndex].LockCount--;
			check(CachedChunks[BufferIndex].LockCount >= 0);
		}
	}
}

FEvent* FChunkCacheWorker::AcquireNotificationEvent() const
{
	return FPlatformProcess::GetSynchEventFromPool();
}

void FChunkCacheWorker::ReleaseNotificationEvent(FEvent* Event)
{
	check(Event != nullptr);

	EventsToRelease.Push(Event);

	// Wake up the thread if it needs it
	if (QueuedRequestsEvent)
	{
		QueuedRequestsEvent->Trigger();
	}
}

int32 FChunkCacheWorker::ProcessQueue()
{
	SCOPE_SECONDS_ACCUMULATOR(STAT_FChunkCacheWorker_ProcessQueue);
	INC_DWORD_STAT(STAT_FChunkCacheWorker_NumProcessQueue);

	// Add the queue to the active requests list
	if (PendingQueueCounter.GetValue() > 0)
	{
		SCOPE_SECONDS_ACCUMULATOR(STAT_FChunkCacheWorker_RequestQueueUpdate);

		FScopeLock LockQueue(&QueueLock);
		ActiveRequests.Append(RequestQueue);
		PendingQueueCounter.Subtract(RequestQueue.Num());
		RequestQueue.Empty();
	}

	// Keep track how many request have been process this loop
	int32 ProcessedRequests = ActiveRequests.Num();

	if (ProcessedRequests)
	{
		INC_DWORD_STAT(STAT_FChunkCacheWorker_NumProcessQueueWithWork);
	}

	bool bFreedUpResourceWithoutHandingItOut = false;
	for (int32 RequestIndex = 0; RequestIndex < ActiveRequests.Num(); ++RequestIndex)
	{
		FChunkRequest& Request = *ActiveRequests[RequestIndex];
		if (Request.RefCount.GetValue() == 0)
		{
			// ChunkRequest is no longer used by anything. Add it to the free requests lists
			// and release the associated buffer.
			ReleaseBuffer(Request.Index);
			ActiveRequests.RemoveAt(RequestIndex--);
			FreeChunkRequests.Push(&Request);

			// If we free a resource and don't hand it out in any follow-up requests after we free it, then trigger
			// our event to run ProcessQueue again and to see if any earlier requests could use it.
			bFreedUpResourceWithoutHandingItOut = true;
		}
		else if (Request.Buffer == NULL)
		{
			// See if the requested chunk is already cached.
			FChunkBuffer* CachedBuffer = GetCachedChunkBuffer(Request.Index);
			if (!CachedBuffer)
			{
				// This chunk is not cached. Get a free buffer if possible.
				CachedBuffer = GetFreeBuffer();
				if (!!CachedBuffer)
				{
					// Load and verify.
					bFreedUpResourceWithoutHandingItOut = false;
					CachedBuffer->ChunkIndex = Request.Index;
					Request.Buffer = CachedBuffer;
					CheckSignature(Request);
				}
			}
			else
			{
				Request.Buffer = CachedBuffer;
			}
			
			if (!!CachedBuffer)
			{
				check(Request.Buffer == CachedBuffer);
				// Chunk is cached and trusted. We no longer need the request handle on this thread.
				// Let the other thread know the chunk is ready to read.
				Request.RefCount.Decrement();
				Request.IsTrusted.Increment();

				if (Request.Event != nullptr)
				{
					Request.Event->Trigger();
				}
			}
		}
	}
	if (bFreedUpResourceWithoutHandingItOut && QueuedRequestsEvent)
	{
		QueuedRequestsEvent->Trigger();
	}

	// Release any pending events
	if (!EventsToRelease.IsEmpty())
	{
		while (FEvent* EventToRelease = EventsToRelease.Pop())
		{
			FPlatformProcess::ReturnSynchEventToPool(EventToRelease);
		}
	}

	return ProcessedRequests;
}

bool FChunkCacheWorker::CheckSignature(const FChunkRequest& ChunkInfo)
{
	SCOPE_SECONDS_ACCUMULATOR(STAT_FChunkCacheWorker_CheckSignature);

	bool bChunkHashesMatch = false;
	check(Signatures.IsValid());

	// If our signature data wasn't validated properly on startup, we shouldn't be in here. Mark all chunk checks as failed.
	if (ensure(IsValid()))
	{
		TPakChunkHash ChunkHash;

		{
			SCOPE_SECONDS_ACCUMULATOR(STAT_FChunkCacheWorker_Serialize);
			Reader->Seek(ChunkInfo.Offset);
			Reader->Serialize(ChunkInfo.Buffer->Data, ChunkInfo.Size);
		}
		{
			SCOPE_SECONDS_ACCUMULATOR(STAT_FChunkCacheWorker_HashBuffer);
			ChunkHash = ComputePakChunkHash(ChunkInfo.Buffer->Data, ChunkInfo.Size);
		}

		bChunkHashesMatch = IsValid() && (ChunkHash == Signatures->ChunkHashes[ChunkInfo.Index]);
		if (!bChunkHashesMatch)
		{
			UE_LOGF(LogPakFile, Warning, "Pak chunk signing mismatch on chunk [%i/%i]! Expected %ls, Received %ls", ChunkInfo.Index, Signatures->ChunkHashes.Num() - 1, *ChunkHashToString(Signatures->ChunkHashes[ChunkInfo.Index]), *ChunkHashToString(ChunkHash));

			if (Signatures->DecryptedHash != Signatures->ComputeCurrentPrincipalHash())
			{
				UE_LOGF(LogPakFile, Warning, "Principal signature table has changed since initialization!");
			}

			const FPakChunkSignatureCheckFailedData Data(Reader->GetArchiveName(), Signatures->ChunkHashes[ChunkInfo.Index], ChunkHash, ChunkInfo.Index);
			FPakPlatformFile::BroadcastPakChunkSignatureCheckFailure(Data);
		}
	}
	else
	{
		FMemory::Memset(ChunkInfo.Buffer->Data, 0xcd, ChunkInfo.Size);
	}
	
	return bChunkHashesMatch;
}

FChunkRequest& FChunkCacheWorker::RequestChunk(int32 ChunkIndex, int64 StartOffset, int64 ChunkSize, FEvent* Event)
{
	FChunkRequest* NewChunk = FreeChunkRequests.Pop();
	if (NewChunk == NULL)
	{
		NewChunk = new FChunkRequest();
	}
	NewChunk->Index = ChunkIndex;
	NewChunk->Offset = StartOffset;
	NewChunk->Size = ChunkSize;
	NewChunk->Buffer = NULL;
	NewChunk->IsTrusted.Set(0);
	// At this point both worker and the archive use this chunk so increase ref count
	NewChunk->RefCount.Set(2);
	NewChunk->Event = Event;

	QueueLock.Lock();
	RequestQueue.Add(NewChunk);
	PendingQueueCounter.Increment();
	QueueLock.Unlock();
	if (QueuedRequestsEvent)
	{
		QueuedRequestsEvent->Trigger();
	}
	
	return *NewChunk;
}

bool FChunkCacheWorker::IsValid() const
{
	return Signatures.IsValid() && (Signatures->ChunkHashes.Num() > 0);
}

void FChunkCacheWorker::ReleaseChunk(FChunkRequest& Chunk)
{
	if (Chunk.RefCount.Decrement() == 0 && QueuedRequestsEvent != nullptr)
	{
		QueuedRequestsEvent->Trigger();
	}
}

FSignedArchiveReader::FSignedArchiveReader(FArchive* InPakReader, FChunkCacheWorker* InSignatureChecker)
	: ChunkCount(0)
	, PakReader(InPakReader)
	, SizeOnDisk(0)
	, PakSize(0)
	, PakOffset(0)
	, SignatureChecker(InSignatureChecker)
{
	// Cache global info about the archive
	this->SetIsLoading(true);
	SizeOnDisk = PakReader->TotalSize();
	ChunkCount = SizeOnDisk / FPakInfo::MaxChunkDataSize + ((SizeOnDisk % FPakInfo::MaxChunkDataSize) ? 1 : 0);
	PakSize = SizeOnDisk;
}

FSignedArchiveReader::~FSignedArchiveReader()
{
	delete PakReader;
	PakReader = NULL;
}

int64 FSignedArchiveReader::CalculateChunkSize(int64 ChunkIndex) const
{
	if (ChunkIndex == (ChunkCount - 1))
	{
		const int64 MaxChunkSize = FPakInfo::MaxChunkDataSize;
		int64 Slack = SizeOnDisk % MaxChunkSize;
		if (!Slack)
		{
			return FPakInfo::MaxChunkDataSize;
		}
		else
		{
			check(Slack > 0);
			return Slack;
		}
	}
	else
	{
		return FPakInfo::MaxChunkDataSize;
	}
}

int64 FSignedArchiveReader::PrecacheChunks(TArray<FSignedArchiveReader::FReadInfo>& Chunks, int64 Length, FEvent* Event)
{
	SCOPE_SECONDS_ACCUMULATOR(STAT_SignedArchiveReader_PreCacheChunks);

	// Request all the chunks that are needed to complete this read
	int64 DataOffset = 0;
	int64 DestOffset = 0;
	const int64 FirstChunkIndex = CalculateChunkIndex(PakOffset);
	const int64 LastChunkIndex = CalculateChunkIndex(PakOffset + Length - 1);
	const int64 NumChunks = LastChunkIndex - FirstChunkIndex + 1;
	int64 ChunkStartOffset = 0;
	int64 RemainingLength = Length;
	int64 ArchiveOffset = PakOffset;

	Chunks.Empty(IntCastChecked<int32>(NumChunks));
	for (int32 ChunkIndex = IntCastChecked<int32>(FirstChunkIndex); ChunkIndex <= LastChunkIndex; ++ChunkIndex)
	{
		ChunkStartOffset = RemainingLength > 0 ? CalculateChunkOffset(ArchiveOffset, DataOffset) : CalculateChunkOffsetFromIndex(ChunkIndex);
		int64 SizeToReadFromBuffer = RemainingLength;
		if (DataOffset + SizeToReadFromBuffer > ChunkStartOffset + FPakInfo::MaxChunkDataSize)
		{
			SizeToReadFromBuffer = ChunkStartOffset + FPakInfo::MaxChunkDataSize - DataOffset;
		}

		FReadInfo ChunkInfo;
		ChunkInfo.SourceOffset = DataOffset - ChunkStartOffset;
		ChunkInfo.DestOffset = DestOffset;
		ChunkInfo.Size = SizeToReadFromBuffer;

		if (LastCachedChunk.ChunkIndex == ChunkIndex)
		{
			ChunkInfo.Request = NULL;
			ChunkInfo.PreCachedChunk = &LastCachedChunk;
		}
		else
		{
			const int64 ChunkSize = CalculateChunkSize(ChunkIndex);	
			ChunkInfo.Request = &SignatureChecker->RequestChunk(ChunkIndex, ChunkStartOffset, ChunkSize, Event);
			INC_DWORD_STAT(STAT_SignedArchiveReader_NumChunkRequests);
			ChunkInfo.PreCachedChunk = NULL;
		}

		Chunks.Add(ChunkInfo);

		ArchiveOffset += SizeToReadFromBuffer;
		DestOffset += SizeToReadFromBuffer;
		RemainingLength -= SizeToReadFromBuffer;
	}

	return NumChunks;
}

void FSignedArchiveReader::Serialize(void* Data, int64 Length)
{
	SCOPE_SECONDS_ACCUMULATOR(STAT_SignedArchiveReader_Serialize);
	INC_DWORD_STAT(STAT_SignedArchiveReader_NumSerializes);

	FEvent* ChunkReadEvent = SignatureChecker->IsMultithreaded() ? SignatureChecker->AcquireNotificationEvent() : nullptr;
	
	// First make sure the chunks we're going to read are actually cached.
	TArray<FReadInfo> QueuedChunks;
	int32 ChunksToRead = IntCastChecked<int32>(PrecacheChunks(QueuedChunks, Length, ChunkReadEvent));
	int32 FirstPrecacheChunkIndex = ChunksToRead;

	// If we aren't multithreaded then flush the signature checking now so there will be some data ready
	// for us in the loop
	if (!SignatureChecker->IsMultithreaded())
	{
		SignatureChecker->ProcessQueue();
	}

	// Read data from chunks.
	int64 RemainingLength = Length;
	uint8* DestData = (uint8*)Data;

	{
		SCOPE_SECONDS_ACCUMULATOR(STAT_SignedArchiveReader_ProcessChunkRequests);
		const int32 LastRequestIndex = ChunksToRead - 1;
		do
		{
			int32 ChunksReadThisLoop = 0;
			// Try to read cached chunks. If a chunk is not yet ready, skip to the next chunk - it's possible
			// that it has already been precached in one of the previous reads.
			for (int32 QueueIndex = 0; QueueIndex <= LastRequestIndex; ++QueueIndex)
			{
				FReadInfo& ChunkInfo = QueuedChunks[QueueIndex];
				if (ChunkInfo.Request && ChunkInfo.Request->IsReady())
				{
					SCOPE_SECONDS_ACCUMULATOR(STAT_SignedArchiveReader_CopyFromNewCache);

					// Read
					FMemory::Memcpy(DestData + ChunkInfo.DestOffset, ChunkInfo.Request->Buffer->Data + ChunkInfo.SourceOffset, ChunkInfo.Size);
					// Is this the last chunk? if so, copy it to pre-cache
					if (LastRequestIndex == QueueIndex && ChunkInfo.Request->Index != LastCachedChunk.ChunkIndex)
					{
						LastCachedChunk.ChunkIndex = ChunkInfo.Request->Index;
						FMemory::Memcpy(LastCachedChunk.Data, ChunkInfo.Request->Buffer->Data, FPakInfo::MaxChunkDataSize);
					}
					// Let the worker know we're done with this chunk for now.
					SignatureChecker->ReleaseChunk(*ChunkInfo.Request);
					ChunkInfo.Request = NULL;
					// One less chunk remaining
					ChunksToRead--;
					ChunksReadThisLoop++;
				}
				else if (ChunkInfo.PreCachedChunk)
				{
					SCOPE_SECONDS_ACCUMULATOR(STAT_SignedArchiveReader_CopyFromExistingCache);

					// Copy directly from the pre-cached chunk.
					FMemory::Memcpy(DestData + ChunkInfo.DestOffset, ChunkInfo.PreCachedChunk->Data + ChunkInfo.SourceOffset, ChunkInfo.Size);
					ChunkInfo.PreCachedChunk = NULL;
					// One less chunk remaining
					ChunksToRead--;
					ChunksReadThisLoop++;
				}
			}

			if (ChunksReadThisLoop == 0)
			{
				if (ChunkReadEvent != nullptr)
				{
					ChunkReadEvent->Wait();
				}
				else
				{
					// Process some more buffers
					SignatureChecker->ProcessQueue();
				}
			}
		}
		while (ChunksToRead > 0);
	}

	if (ChunkReadEvent != nullptr)
	{
		SignatureChecker->ReleaseNotificationEvent(ChunkReadEvent);
		ChunkReadEvent = nullptr;
	}

	PakOffset += Length;

	// Free precached chunks (they will still get precached but simply marked as not used by anything)
	for (int32 QueueIndex = FirstPrecacheChunkIndex; QueueIndex < QueuedChunks.Num(); ++QueueIndex)
	{
		FReadInfo& CachedChunk = QueuedChunks[QueueIndex];
		if (CachedChunk.Request)
		{
			SignatureChecker->ReleaseChunk(*CachedChunk.Request);
		}
	}
}
