// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/ThreadSafeCounter.h"
#include "Containers/LockFreeList.h"
#include "IPlatformFilePak.h"
#include "HAL/PlatformProcess.h"
#include "HAL/Runnable.h"
#include "Math/BigInt.h"

/**
 * Chunk buffer.
 * Buffers are locked and released only on the worker thread.
 */
struct FChunkBuffer
{
	/** Chunk data */
	uint8* Data;
	/** Number of locks on this buffer */
	int32 LockCount;
	/** Index of the chunk */
	int32 ChunkIndex;
	/** Last time this buffer has been accessed */
	double LastAccessTime;

	FChunkBuffer()
		: Data(NULL)
		, LockCount(0)
		, ChunkIndex(INDEX_NONE)
		, LastAccessTime(0.0)
	{
		Data = (uint8*)FMemory::Malloc(FPakInfo::MaxChunkDataSize);
	}

	~FChunkBuffer()
	{
		FMemory::Free(Data);
		Data = NULL;
	}
};

/**
 * Request to load a chunk. This is how the archive reader and the worker thread
 * communicate. Requests can be locked by both threads.
 */
struct FChunkRequest
{
	/** Chunk index */
	int32 Index;
	/** Chunk offset */
	int64 Offset;
	/** Chunk size */
	int64 Size;
	/** Buffer where the data is cached */
	FChunkBuffer* Buffer;
	/** Flag to indicate if the chunk has been verified */
	FThreadSafeCounter IsTrusted;
	/** Reference count */
	FThreadSafeCounter RefCount;
	/** Optional pointer to an FEvent that can be used to signal that the request is ready for use */
	FEvent* Event;

	/**
	 * Constructor
	 */
	FChunkRequest()
		: Index(INDEX_NONE)
		, Offset(0)
		, Size(0)
		, Buffer(nullptr)
		, IsTrusted(0)
		, RefCount(0)
		, Event(nullptr)
	{}

	/**
	 * Waits until this chunk has been verified
	 */
	FORCEINLINE void WaitUntilReady() const
	{
		while (IsTrusted.GetValue() == 0)
		{
			FPlatformProcess::Sleep(0.0f);
		}
	}
	/**
	 * Checks if this chunk has been verified.
	 */
	FORCEINLINE bool IsReady() const
	{
		return IsTrusted.GetValue() != 0;
	}
};

/**
 * Thread that loads and verifies signed chunks.
 * One per pak file but can serve multiple FSignedArchiveReaders from multiple threads!
 * Can process multiple chunks using a limited number of buffers.
 */
class FChunkCacheWorker : public FRunnable
{	
	enum
	{
		/** Buffer size */
		MaxCachedChunks = 8		
	};

	/** Reference hashes */
	TSharedPtr<const FPakSignatureFile, ESPMode::ThreadSafe> Signatures;
	/** Hash of the sig file data. Used to check that nothing was corrupted when a signature check fails */
	TPakChunkHash OriginalSignatureFileHash;
	/** Thread to run the worker FRunnable on */
	FRunnableThread* Thread;
	/** Archive reader */
	TUniquePtr<FArchive> Reader;
	/** Cached and verified chunks. */
	FChunkBuffer CachedChunks[MaxCachedChunks];
	/** Queue of chunks to cache */
	TArray<FChunkRequest*> RequestQueue;	
	/** Lock for manipulating the queue */
	FCriticalSection QueueLock;
	/** Counter indicating how many pending queued request exist */
	FThreadSafeCounter PendingQueueCounter;
	/** Event used to signal there's work to be done */
	FEvent* QueuedRequestsEvent;
	/** List of active chunk requests */
	TArray<FChunkRequest*> ActiveRequests;
	/** Stops this thread */
	FThreadSafeCounter StopTaskCounter;
	/** Available chunk requests */
	TLockFreePointerListUnordered<FChunkRequest, PLATFORM_CACHE_LINE_SIZE> FreeChunkRequests;
	/** List of FEvents to be released on the worker thread */
	TLockFreePointerListUnordered<FEvent,PLATFORM_CACHE_LINE_SIZE> EventsToRelease;

	/** 
	 * Process requested chunks 
	 *
	 * @return Number of chunks processed this loop.
	 */
	int32 ProcessQueue();
	/** 
	 * Verifies chunk signature [*]
	 */
	bool CheckSignature(const FChunkRequest& ChunkInfo);
	/** 
	 * Tries to get a pre-cached chunk buffer 
	 */
	FChunkBuffer* GetCachedChunkBuffer(int32 ChunkIndex);
	/** 
	 * Tries to get the least recent free buffer 
	 */
	FChunkBuffer* GetFreeBuffer();
	/** 
	 * Decrements a ref count on a buffer for the specified chunk 
	 */
	void ReleaseBuffer(int32 ChunkIndex);
	/**
	* Is this chunk cache worker running in a thread?
	*/
	FORCEINLINE bool IsMultithreaded() const 
	{ 
		return Thread != nullptr;
	}
	/**
	 * Returns a FEvent that can later be released via ReleaseNotificationEvent
	 */
	FEvent* AcquireNotificationEvent() const;
	/**
	 * Queues the event to be released safely on the FChunkCacheWorker thread
	 */
	void ReleaseNotificationEvent(FEvent* Event);

public:

	FChunkCacheWorker(TUniquePtr<FArchive> InReader, const TCHAR* Filename);
	virtual ~FChunkCacheWorker();

	//~ Begin FRunnable Interface.
	virtual bool Init();
	virtual uint32 Run();
	virtual void Stop();
	//~ End FRunnable Interface

	TSharedPtr<const FPakSignatureFile, ESPMode::ThreadSafe> GetSignatures() const
	{
		return Signatures;
	}

	/** 
	 * Requests a chunk to be loaded and verified
	 * 
	 * @param ChunkIndex Index of a chunk to load
	 * @param StartOffset Offset to the beginning of the chunk
	 * @param ChunkSize Chunk size
	 * @param Event Optional FEvent that will signal when the request is ready, nullptr is valid if the calling code does not want this signal
	 * @return Handle to the request.
	 */
	FChunkRequest& RequestChunk(int32 ChunkIndex, int64 StartOffset, int64 ChunkSize, FEvent* Event);
	/**
	 * Releases the requested chunk buffer
	 */
	void ReleaseChunk(FChunkRequest& Chunk);


	/**
	* Indicates that this chunk worker is valid. If the signature file couldn't be loaded or if it failed
	* the principal table check, this will be false
	*/
	bool IsValid() const;

	friend class FSignedArchiveReader;
};

/////////////////////////////////////////////////////////////////////////////////////////

/**
 * FSignedArchiveReader - reads data from pre-cached and verified chunks.
 */
class FSignedArchiveReader : public FArchive
{
	struct FReadInfo
	{
		FChunkRequest* Request;
		FChunkBuffer* PreCachedChunk;
		int64 SourceOffset;
		int64 DestOffset;
		int64 Size;
	};

	/** Number of chunks in the archive */
	int64 ChunkCount;	
	/** Reader archive */
	FArchive* PakReader;
	/** Size of the archive on disk */
	int64 SizeOnDisk;
	/** Size of actual data (excluding signatures) */
	int64 PakSize;
	/** Current offet into data */
	int64 PakOffset;
	/** Worker thread - reads chunks from disk and verifies their signatures */
	FChunkCacheWorker* SignatureChecker;
	/** Last pre-cached buffer */
	FChunkBuffer LastCachedChunk;

	/** 
	 * Calculate index of a chunk that contains the specified offset 
	 */
	FORCEINLINE int64 CalculateChunkIndex(int64 ReadOffset) const
	{
		return (ReadOffset / FPakInfo::MaxChunkDataSize);
	}	

	/** 
	 * Calculate offset of a chunk in the archive 
	 */
	FORCEINLINE int64 CalculateChunkOffsetFromIndex(int64 BufferIndex) const
	{
		return BufferIndex * FPakInfo::MaxChunkDataSize;
	}

	/** 
	 * Calculate offset of a chunk in the archive and the offset to read from the archive 
	 *
	 * @param ReadOffset Read request offset
	 * @param OutDataOffset Actuall offset to read from in the archive
	 * @return Offset where the chunk begins in the archive (signature offset)
	 */
	FORCEINLINE int64 CalculateChunkOffset(int64 ReadOffset, int64& OutDataOffset) const
	{
		const int64 ChunkIndex = CalculateChunkIndex(ReadOffset);
		OutDataOffset = ReadOffset;
		return CalculateChunkOffsetFromIndex(ChunkIndex);
	}
	
	/** 
	 * Calculates chunk size based on its index (most chunks have the same size, except the last one 
	 */
	int64 CalculateChunkSize(int64 ChunkIndex) const;

	/** 
	 * Queues chunks on the worker thread 
	 * @param Chunks This array will contain info about each chunk created by the call.
	 * @param Length The length of data to precache
	 * @param Event Optional FEvent that will signal as each request becomes ready, nullptr is valid if the calling code does not want this signal
	 * @return Number of chunks in the output array which are actually required for the requested length. The rest are precache chunks 
	 */
	int64 PrecacheChunks(TArray<FReadInfo>& Chunks, int64 Length, FEvent* Event);

public:

	FSignedArchiveReader(FArchive* InPakReader, FChunkCacheWorker* InSignatureChecker);
	virtual ~FSignedArchiveReader();

	//~ Begin FArchive Interface
	virtual void Serialize(void* Data, int64 Length) override;
	virtual int64 Tell() override
	{
		return PakOffset;
	}
	virtual int64 TotalSize() override
	{
		return PakSize;
	}
	virtual void Seek(int64 InPos) override
	{
		PakOffset = InPos;
	}
	//~ End FArchive Interface
};
