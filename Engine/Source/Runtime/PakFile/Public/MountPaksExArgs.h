// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Experimental/UnifiedError/UnifiedError.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "Templates/ValueOrError.h"

namespace UE
{
	struct FMountPaksExArgs
	{
		struct FMountResult
		{
			// Any pak file that was mounted, may be null on success if an encryption key is pending
			IPakFile* PakFile = nullptr;
		};

		const TCHAR* PakFilePath = nullptr;
		int32 Order = INDEX_NONE;
		FPakMountOptions MountOptions;
		TValueOrError<FMountResult, UE::UnifiedError::FError> Result = MakeValue();
	};
}
